{
	"FileVersion": 3,
	"Version": 1,
	"VersionName": "0.9",
	"FriendlyName": "Compute Framework",
	"Description": "Support for user authored GPU compute graphs",
	"Category": "Rendering",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"DocsURL": "",
	"MarketplaceURL": "",
	"SupportURL": "",
	"EnabledByDefault": false,
	"CanContainContent": false,
	"IsBetaVersion": true,
	"Installed": false,
	"Modules": [
		{
			"Name": "ComputeFramework",
			"Type": "Runtime",
			"LoadingPhase": "PostConfigInit"
		},
		{
			"Name": "ComputeFrameworkEditor",
			"Type": "Editor",
			"LoadingPhase": "Default"
		},
		{
			"Name": "ComputeDataInterface",
			"Type": "Runtime",
			"LoadingPhase": "Default"
		},
		{
			"Name": "EditableComputeGraph",
			"Type": "Runtime",
			"LoadingPhase": "Default"
		},
		{
			"Name": "EditableComputeGraphEditor",
			"Type": "Editor",
			"LoadingPhase": "Default"
		}
	]
}
