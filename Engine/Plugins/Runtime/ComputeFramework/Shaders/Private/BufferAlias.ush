// Copyright Epic Games, Inc. All Rights Reserved.

// Implement ReadRawBuffer/WriteRawBuffer functions for plain old types that need it.
// Due to restrictions on some platforms we can't use StructureBuffer with unaligned types. That includes all vector<T, 3> types.
// Similarly user structure types that don't follow alignment restrictions might have to implement similar functions to read from raw buffers.

#pragma once

void ReadRawBuffer(in StructuredBuffer<float> InBuffer, in uint InBufferIndex, out float3 OutValue)
{
	OutValue.x = InBuffer[InBufferIndex * 3 + 0];
	OutValue.y = InBuffer[InBufferIndex * 3 + 1];
	OutValue.z = InBuffer[InBufferIndex * 3 + 2];
}
void ReadRawBuffer(in RWStructuredBuffer<float> InBuffer, in uint InBufferIndex, out float3 OutValue)
{
	OutValue.x = InBuffer[InBufferIndex * 3 + 0];
	OutValue.y = InBuffer[InBufferIndex * 3 + 1];
	OutValue.z = InBuffer[InBufferIndex * 3 + 2];
}
void WriteRawBuffer(in RWStructuredBuffer<float> InBuffer, in uint InBufferIndex, in float3 InValue)
{
	InBuffer[InBufferIndex * 3 + 0] = InValue.x;
	InBuffer[InBufferIndex * 3 + 1] = InValue.y;
	InBuffer[InBufferIndex * 3 + 2] = InValue.z;
}

void ReadRawBuffer(in StructuredBuffer<int> InBuffer, in uint InBufferIndex, out int3 OutValue)
{
	OutValue.x = InBuffer[InBufferIndex * 3 + 0];
	OutValue.y = InBuffer[InBufferIndex * 3 + 1];
	OutValue.z = InBuffer[InBufferIndex * 3 + 2];
}
void ReadRawBuffer(in RWStructuredBuffer<int> InBuffer, in uint InBufferIndex, out int3 OutValue)
{
	OutValue.x = InBuffer[InBufferIndex * 3 + 0];
	OutValue.y = InBuffer[InBufferIndex * 3 + 1];
	OutValue.z = InBuffer[InBufferIndex * 3 + 2];
}
void WriteRawBuffer(in RWStructuredBuffer<int> InBuffer, in uint InBufferIndex, in int3 InValue)
{
	InBuffer[InBufferIndex * 3 + 0] = InValue.x;
	InBuffer[InBufferIndex * 3 + 1] = InValue.y;
	InBuffer[InBufferIndex * 3 + 2] = InValue.z;
}
