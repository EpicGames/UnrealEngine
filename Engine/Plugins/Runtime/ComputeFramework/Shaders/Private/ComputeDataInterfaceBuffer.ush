// Copyright Epic Games, Inc. All Rights Reserved.

// VALUE_TYPE is the exposed type that calling code expects to use.
#define VALUE_TYPE {ValueType}
#define VALUE_TYPE_STRIDE_BYTES {ValueTypeStride}
// VALUE_TYPE_SUPPORTS_ATOMIC is set if we can enable atomic operations on the VALUE_TYPE.
#define VALUE_TYPE_SUPPORTS_ATOMIC {SupportAtomic}
// BUFFER_SPLIT_READ_WRITE is set if we can use an SRV to read the buffer. It is unset if we might need to read and write the buffer in the same kernel.
#define BUFFER_SPLIT_READ_WRITE {SplitReadWrite}

uint {DataInterfaceName}_BufferElementCount;
#if BUFFER_SPLIT_READ_WRITE
ByteAddressBuffer {DataInterfaceName}_BufferSRV;
#endif
RWByteAddressBuffer {DataInterfaceName}_BufferUAV;

void ReadBufferSRV_{DataInterfaceName}(in uint Index, out uint ReturnValue)
{
	ReturnValue = {DataInterfaceName}_BufferSRV.Load(Index * VALUE_TYPE_STRIDE_BYTES);
}
void ReadBufferSRV_{DataInterfaceName}(in uint Index, out uint2 ReturnValue)
{
	ReturnValue = {DataInterfaceName}_BufferSRV.Load2(Index * VALUE_TYPE_STRIDE_BYTES);
}
void ReadBufferSRV_{DataInterfaceName}(in uint Index, out uint3 ReturnValue)
{
	ReturnValue = {DataInterfaceName}_BufferSRV.Load3(Index * VALUE_TYPE_STRIDE_BYTES);
}
void ReadBufferSRV_{DataInterfaceName}(in uint Index, out uint4 ReturnValue)
{
	ReturnValue = {DataInterfaceName}_BufferSRV.Load4(Index * VALUE_TYPE_STRIDE_BYTES);
}
void ReadBufferUAV_{DataInterfaceName}(in uint Index, out uint ReturnValue)
{
	ReturnValue = {DataInterfaceName}_BufferSRV.Load(Index * VALUE_TYPE_STRIDE_BYTES);
}
void ReadBufferUAV_{DataInterfaceName}(in uint Index, out uint2 ReturnValue)
{
	ReturnValue = {DataInterfaceName}_BufferSRV.Load2(Index * VALUE_TYPE_STRIDE_BYTES);
}
void ReadBufferUAV_{DataInterfaceName}(in uint Index, out uint3 ReturnValue)
{
	ReturnValue = {DataInterfaceName}_BufferSRV.Load3(Index * VALUE_TYPE_STRIDE_BYTES);
}
void ReadBufferUAV_{DataInterfaceName}(in uint Index, out uint4 ReturnValue)
{
	ReturnValue = {DataInterfaceName}_BufferSRV.Load4(Index * VALUE_TYPE_STRIDE_BYTES);
}
void WriteBuffer_{DataInterfaceName}(in uint Index, in uint Value)
{
	{DataInterfaceName}_BufferUAV.Store(Index * VALUE_TYPE_STRIDE_BYTES, Value);
}
void WriteBuffer_{DataInterfaceName}(in uint Index, in uint2 Value)
{
	{DataInterfaceName}_BufferUAV.Store2(Index * VALUE_TYPE_STRIDE_BYTES, Value);
}
void WriteBuffer_{DataInterfaceName}(in uint Index, in uint3 Value)
{
	{DataInterfaceName}_BufferUAV.Store3(Index * VALUE_TYPE_STRIDE_BYTES, Value);
}
void WriteBuffer_{DataInterfaceName}(in uint Index, in uint4 Value)
{
	{DataInterfaceName}_BufferUAV.Store4(Index * VALUE_TYPE_STRIDE_BYTES, Value);
}

void ReadBufferSRV_{DataInterfaceName}(in uint Index, out int ReturnValue)
{
	ReturnValue = asint({DataInterfaceName}_BufferSRV.Load(Index * VALUE_TYPE_STRIDE_BYTES));
}
void ReadBufferSRV_{DataInterfaceName}(in uint Index, out int2 ReturnValue)
{
	ReturnValue = asint({DataInterfaceName}_BufferSRV.Load2(Index * VALUE_TYPE_STRIDE_BYTES));
}
void ReadBufferSRV_{DataInterfaceName}(in uint Index, out int3 ReturnValue)
{
	ReturnValue = asint({DataInterfaceName}_BufferSRV.Load3(Index * VALUE_TYPE_STRIDE_BYTES));
}
void ReadBufferSRV_{DataInterfaceName}(in uint Index, out int4 ReturnValue)
{
	ReturnValue = asint({DataInterfaceName}_BufferSRV.Load4(Index * VALUE_TYPE_STRIDE_BYTES));
}
void ReadBufferUAV_{DataInterfaceName}(in uint Index, out int ReturnValue)
{
	ReturnValue = asint({DataInterfaceName}_BufferSRV.Load(Index * VALUE_TYPE_STRIDE_BYTES));
}
void ReadBufferUAV_{DataInterfaceName}(in uint Index, out int2 ReturnValue)
{
	ReturnValue = asint({DataInterfaceName}_BufferSRV.Load2(Index * VALUE_TYPE_STRIDE_BYTES));
}
void ReadBufferUAV_{DataInterfaceName}(in uint Index, out int3 ReturnValue)
{
	ReturnValue = asint({DataInterfaceName}_BufferSRV.Load3(Index * VALUE_TYPE_STRIDE_BYTES));
}
void ReadBufferUAV_{DataInterfaceName}(in uint Index, out int4 ReturnValue)
{
	ReturnValue = asint({DataInterfaceName}_BufferSRV.Load4(Index * VALUE_TYPE_STRIDE_BYTES));
}
void WriteBuffer_{DataInterfaceName}(in uint Index, in int Value)
{
	{DataInterfaceName}_BufferUAV.Store(Index * VALUE_TYPE_STRIDE_BYTES, asuint(Value));
}
void WriteBuffer_{DataInterfaceName}(in uint Index, in int2 Value)
{
	{DataInterfaceName}_BufferUAV.Store2(Index * VALUE_TYPE_STRIDE_BYTES, asuint(Value));
}
void WriteBuffer_{DataInterfaceName}(in uint Index, in int3 Value)
{
	{DataInterfaceName}_BufferUAV.Store3(Index * VALUE_TYPE_STRIDE_BYTES, asuint(Value));
}
void WriteBuffer_{DataInterfaceName}(in uint Index, in int4 Value)
{
	{DataInterfaceName}_BufferUAV.Store4(Index * VALUE_TYPE_STRIDE_BYTES, asuint(Value));
}

void ReadBufferSRV_{DataInterfaceName}(in uint Index, out float ReturnValue)
{
	ReturnValue = asfloat({DataInterfaceName}_BufferSRV.Load(Index * VALUE_TYPE_STRIDE_BYTES));
}
void ReadBufferSRV_{DataInterfaceName}(in uint Index, out float2 ReturnValue)
{
	ReturnValue = asfloat({DataInterfaceName}_BufferSRV.Load2(Index * VALUE_TYPE_STRIDE_BYTES));
}
void ReadBufferSRV_{DataInterfaceName}(in uint Index, out float3 ReturnValue)
{
	ReturnValue = asfloat({DataInterfaceName}_BufferSRV.Load3(Index * VALUE_TYPE_STRIDE_BYTES));
}
void ReadBufferSRV_{DataInterfaceName}(in uint Index, out float4 ReturnValue)
{
	ReturnValue = asfloat({DataInterfaceName}_BufferSRV.Load4(Index * VALUE_TYPE_STRIDE_BYTES));
}
void ReadBufferUAV_{DataInterfaceName}(in uint Index, out float ReturnValue)
{
	ReturnValue = asfloat({DataInterfaceName}_BufferSRV.Load(Index * VALUE_TYPE_STRIDE_BYTES));
}
void ReadBufferUAV_{DataInterfaceName}(in uint Index, out float2 ReturnValue)
{
	ReturnValue = asfloat({DataInterfaceName}_BufferSRV.Load2(Index * VALUE_TYPE_STRIDE_BYTES));
}
void ReadBufferUAV_{DataInterfaceName}(in uint Index, out float3 ReturnValue)
{
	ReturnValue = asfloat({DataInterfaceName}_BufferSRV.Load3(Index * VALUE_TYPE_STRIDE_BYTES));
}
void ReadBufferUAV_{DataInterfaceName}(in uint Index, out float4 ReturnValue)
{
	ReturnValue = asfloat({DataInterfaceName}_BufferSRV.Load4(Index * VALUE_TYPE_STRIDE_BYTES));
}
void WriteBuffer_{DataInterfaceName}(in uint Index, in float Value)
{
	{DataInterfaceName}_BufferUAV.Store(Index * VALUE_TYPE_STRIDE_BYTES, asuint(Value));
}
void WriteBuffer_{DataInterfaceName}(in uint Index, in float2 Value)
{
	{DataInterfaceName}_BufferUAV.Store2(Index * VALUE_TYPE_STRIDE_BYTES, asuint(Value));
}
void WriteBuffer_{DataInterfaceName}(in uint Index, in float3 Value)
{
	{DataInterfaceName}_BufferUAV.Store3(Index * VALUE_TYPE_STRIDE_BYTES, asuint(Value));
}
void WriteBuffer_{DataInterfaceName}(in uint Index, in float4 Value)
{
	{DataInterfaceName}_BufferUAV.Store4(Index * VALUE_TYPE_STRIDE_BYTES, asuint(Value));
}

uint ReadNumValues_{DataInterfaceName}()
{
	return {DataInterfaceName}_BufferElementCount;
}

VALUE_TYPE ReadValue_{DataInterfaceName}(uint Index)
{
	VALUE_TYPE ReturnValue;
#if BUFFER_SPLIT_READ_WRITE
	ReadBufferSRV_{DataInterfaceName}(Index, ReturnValue);
#else
	ReadBufferUAV_{DataInterfaceName}(Index, ReturnValue);
#endif
	return ReturnValue;
}

VALUE_TYPE ReadValueUAV_{DataInterfaceName}(uint Index)
{
	VALUE_TYPE ReturnValue;
	ReadBufferUAV_{DataInterfaceName}(Index, ReturnValue);
	return ReturnValue;
}

void WriteValue_{DataInterfaceName}(uint Index, VALUE_TYPE Value)
{
	WriteBuffer_{DataInterfaceName}(Index, Value);
}

VALUE_TYPE WriteAtomicAdd_{DataInterfaceName}(uint Index, VALUE_TYPE Value)
{
#if VALUE_TYPE_SUPPORTS_ATOMIC
	VALUE_TYPE OldValue;
	{DataInterfaceName}_BufferUAV.InterlockedAdd(Index * VALUE_TYPE_STRIDE_BYTES, Value, OldValue);
	return OldValue;
#endif
	return 0;
}

VALUE_TYPE WriteAtomicMin_{DataInterfaceName}(uint Index, VALUE_TYPE Value)
{
#if VALUE_TYPE_SUPPORTS_ATOMIC
	VALUE_TYPE OldValue;
	{DataInterfaceName}_BufferUAV.InterlockedMin(Index * VALUE_TYPE_STRIDE_BYTES, Value, OldValue);
	return OldValue;
#endif
	return 0;
}

VALUE_TYPE WriteAtomicMax_{DataInterfaceName}(uint Index, VALUE_TYPE Value)
{
#if VALUE_TYPE_SUPPORTS_ATOMIC
	VALUE_TYPE OldValue;
	{DataInterfaceName}_BufferUAV.InterlockedMax(Index * VALUE_TYPE_STRIDE_BYTES, Value, OldValue);
	return OldValue;
#endif
	return 0;
}

#undef VALUE_TYPE
#undef VALUE_TYPE_STRIDE_BYTES
#undef VALUE_TYPE_SUPPORTS_ATOMIC
#undef BUFFER_SPLIT_READ_WRITE
