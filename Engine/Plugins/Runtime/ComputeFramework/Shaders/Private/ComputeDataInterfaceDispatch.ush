// Copyright Epic Games, Inc. All Rights Reserved.

uint3 {DataInterfaceName}_NumThreads;

uint3 ReadNumThreads_{DataInterfaceName}()
{
	return {DataInterfaceName}_NumThreads;
}
