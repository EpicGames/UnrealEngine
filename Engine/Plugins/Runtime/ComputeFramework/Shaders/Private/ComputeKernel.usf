// Copyright Epic Games, Inc. All Rights Reserved.

#include "/Plugin/ComputeFramework/Private/ComputeKernelCommon.ush"

// Include the generated kernel source.
#include "/Plugin/ComputeFramework/Generated/ComputeKernel.ush"
