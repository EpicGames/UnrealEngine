// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "/Engine/Private/Common.ush"

// Macro to mark a kernel entry point.
// Only one should exist per kernel.

#define KERNEL_ENTRY_POINT(EntryPointName) \
    void EntryPointName( \
        uint Gidx : SV_GroupIndex, \
        uint3 Gid : SV_GroupID, \
        uint3 GTid: SV_GroupThreadID, \
        uint3 DTid : SV_DispatchThreadID)

// Macros to expose external functions for a kernel.
// A kernel assumes these functions exist and are provided by its data providers.

#define KERNEL_EXTERN_READ(Name, ...)
#define KERNEL_EXTERN_WRITE(Name, ...)

// Macro to disambiguate data interface code with a unique id.
// This allows us to include more than one instance of the same data interface.
// This should be used around all uniforms and all local (ie not DI_IMPL) functions.

#define _DI_CAT_IMPL_(A, B) A##B
#define _DI_CAT_(A, B) _DI_CAT_IMPL_(A, B)
#define DI_UNIFORM_LOCAL(Name) _DI_CAT_(DI_UID, Name)
#define DI_FUNCTION_LOCAL(Name) _DI_CAT_(Name, DI_UID)

// Macros to expose functions implemented by a data provider. 
// This makes a local unqiue function name and expands the function prototype.
// We could just use DI_LOCAL, but these macro also gives the possiblity to extract functions by parsing the file.

#define DI_IMPL_READ(Name, ReturnType, ...) ReturnType DI_FUNCTION_LOCAL(Name)(__VA_ARGS__)
#define DI_IMPL_WRITE(Name, ...) void DI_FUNCTION_LOCAL(Name)(__VA_ARGS__)
