// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class ComputeDataInterface : ModuleRules
	{
		public ComputeDataInterface(ReadOnlyTargetRules Target) : base(Target)
		{
			PublicDependencyModuleNames.AddRange(
				new string[]
				{
					"Core",
					"CoreUObject",
					"Engine",
					"ComputeFramework",
				}
			);

			PrivateDependencyModuleNames.AddRange(
				new string[]
				{
					"RenderCore",
					"RHI",
				}
			);
		}
	}
}
