// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ComputeFramework/ComputeDataInterface.h"
#include "ComputeFramework/ComputeDataProvider.h"
#include "ComputeFramework/ShaderParamTypeDefinition.h"

#include "ComputeDataInterfaceBuffer.generated.h"

class FBufferDataInterfaceParameters;

UENUM()
enum class EComputeDataInterfaceBufferType : uint8
{
	Int,
	Uint,
	Float,
	Int2,
	Uint2,
	Float2,
	Int3,
	Uint3,
	Float3,
	Int4,
	Uint4,
	Float4,
};

/** Compute data interface used to own and give access to a GPU buffer. */
UCLASS(MinimalAPI, Category = ComputeFramework)
class UComputeDataInterfaceBuffer : public UComputeDataInterface
{
	GENERATED_BODY()

	//~ Begin UComputeDataInterface Interface
	TCHAR const* GetClassName() const override { return TEXT("Buffer"); }
	void GetSupportedInputs(TArray<FShaderFunctionDefinition>& OutFunctions) const override;
	void GetSupportedOutputs(TArray<FShaderFunctionDefinition>& OutFunctions) const override;
	void GetShaderParameters(TCHAR const* UID, FShaderParametersMetadataBuilder& InOutBuilder, FShaderParametersMetadataAllocations& InOutAllocations) const override;
	TCHAR const* GetShaderVirtualPath() const override;
	void GetShaderHash(FString& InOutKey) const override;
	void GetHLSL(FString& OutHLSL, FString const& InDataInterfaceName) const override;
	UComputeDataProvider* CreateDataProvider() const override;
	//~ End UComputeDataInterface Interface

public:
	/** The value type for the buffer. */
	UPROPERTY(EditAnywhere, Category = Settings)
	EComputeDataInterfaceBufferType ValueType;

	/** Whether to allow read/write access. */
	UPROPERTY(EditAnywhere, Category = Settings)
	bool bAllowReadWrite = false;

private:
	static TCHAR const* TemplateFilePath;
};

/** Compute data provider implementation for UComputeDataInterfaceBuffer. */
UCLASS(MinimalAPI, Blueprintable, Category = ComputeFramework)
class UBufferDataProvider : public UComputeDataProvider
{
	GENERATED_BODY()

	//~ Begin UComputeDataProvider Interface
	void Initialize(int32 InDataInterfaceIndex, UComputeDataInterface const* InDataInterface, UObject* InBinding, uint64 InInputMask, uint64 InOutputMask) override;
	FComputeDataProviderRenderProxy* GetRenderProxy() override;
	//~ End UComputeDataProvider Interface

public:
	/** The value type for the buffer. */
	UPROPERTY()
	EComputeDataInterfaceBufferType ValueType;

	/** The size of the buffer. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = DataInterface)
	int32 ElementCount = 0;

	/** Whether to clear the buffer before use. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = DataInterface)
	bool bClearBeforeUse = false;
};

/** Compute data provider proxy implementation for UComputeDataInterfaceBuffer. */
class FBufferDataProviderProxy : public FComputeDataProviderRenderProxy
{
	//~ Begin FComputeDataProviderRenderProxy Interface
	bool IsValid(FValidationData const& InValidationData) const override;
	void AllocateResources(FRDGBuilder& GraphBuilder, FAllocationData const& InAllocationData) override;
	void GatherDispatchData(FDispatchData const& InDispatchData) override;
	//~ End FComputeDataProviderRenderProxy Interface

public:
	FBufferDataProviderProxy(uint32 InElementStride, uint32 InElementCount, bool bInClearBeforeUse);

private:
	using FParameters = FBufferDataInterfaceParameters;

	const uint32 ElementStride;
	const uint32 ElementCount;
	const bool bClearBeforeUse;

	FRDGBufferRef Buffer;
	FRDGBufferSRVRef BufferSRV;
	FRDGBufferUAVRef BufferUAV;
};
