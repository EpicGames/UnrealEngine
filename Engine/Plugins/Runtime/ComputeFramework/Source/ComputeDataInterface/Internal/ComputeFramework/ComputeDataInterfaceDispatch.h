// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ComputeFramework/ComputeDataInterface.h"
#include "ComputeFramework/ComputeDataProvider.h"

#include "ComputeDataInterfaceDispatch.generated.h"

class FDataInterfaceDispatchParameters;

/** Compute data interface used to control a static dispatch size, and expose thread count to the shader. */
UCLASS(MinimalAPI, Category = ComputeFramework)
class UComputeDataInterfaceDispatch : public UComputeDataInterface
{
	GENERATED_BODY()

	//~ Begin UComputeDataInterface Interface
	TCHAR const* GetClassName() const override { return TEXT("Dispatch"); }
	bool IsExecutionInterface() const override { return true; }
	void GetSupportedInputs(TArray<FShaderFunctionDefinition>& OutFunctions) const override;
	void GetShaderParameters(TCHAR const* UID, FShaderParametersMetadataBuilder& InOutBuilder, FShaderParametersMetadataAllocations& InOutAllocations) const override;
	const TCHAR* GetShaderVirtualPath() const override;
	void GetShaderHash(FString& InOutKey) const override;
	void GetHLSL(FString& OutHLSL, FString const& InDataInterfaceName) const override;
	UComputeDataProvider* CreateDataProvider() const override;
	//~ End UComputeDataInterface Interface

private:
	static const TCHAR* TemplateFilePath;
};

/** Compute data provider implementation for UComputeDataInterfaceDispatch. */
UCLASS(MinimalAPI, Blueprintable, Category = ComputeFramework)
class UDispatchDataProvider : public UComputeDataProvider
{
	GENERATED_BODY()
	
	//~ Begin UComputeDataProvider Interface
	void Initialize(int32 InDataInterfaceIndex, UComputeDataInterface const* InDataInterface, UObject* InBinding, uint64 InInputMask, uint64 InOutputMask) override;
	FComputeDataProviderRenderProxy* GetRenderProxy() override;
	//~ End UComputeDataProvider Interface

public:
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = DataInterface)
	FIntVector ThreadCount = FIntVector(1, 1, 1);
};

/** Compute data provider proxy implementation for UComputeDataInterfaceDispatch. */
class FDispatchDataProviderProxy : public FComputeDataProviderRenderProxy
{
	//~ Begin FComputeDataProviderRenderProxy Interface
	bool IsValid(FValidationData const& InValidationData) const override;
	int32 GetDispatchThreadCount(TArray<FIntVector>& InOutThreadCounts) const override;
	void GatherDispatchData(FDispatchData const& InDispatchData) override;
	//~ End FComputeDataProviderRenderProxy Interface

public:
	FDispatchDataProviderProxy(FIntVector InThreadCount);

private:
	using FParameters = FDataInterfaceDispatchParameters;
	
	FIntVector ThreadCount;
};
