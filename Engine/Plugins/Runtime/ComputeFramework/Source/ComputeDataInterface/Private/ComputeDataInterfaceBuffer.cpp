// Copyright Epic Games, Inc. All Rights Reserved.

#include "ComputeFramework/ComputeDataInterfaceBuffer.h"

#include "ComputeFramework/ComputeGraphComponent.h"
#include "ComputeFramework/ShaderParamTypeDefinition.h"
#include "RenderGraphBuilder.h"
#include "RenderGraphUtils.h"
#include "ShaderCompilerCore.h"
#include "ShaderParameterMetadataBuilder.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ComputeDataInterfaceBuffer)

static FShaderValueTypeHandle GetValueTypeFromEnum(EComputeDataInterfaceBufferType InEnumValueType)
{
	switch (InEnumValueType)
	{
		case EComputeDataInterfaceBufferType::Uint:		return FShaderValueType::Get(EShaderFundamentalType::Uint);
		case EComputeDataInterfaceBufferType::Float:	return FShaderValueType::Get(EShaderFundamentalType::Float);
		case EComputeDataInterfaceBufferType::Int2:		return FShaderValueType::Get(EShaderFundamentalType::Int, 2);
		case EComputeDataInterfaceBufferType::Uint2:	return FShaderValueType::Get(EShaderFundamentalType::Uint, 2);
		case EComputeDataInterfaceBufferType::Float2:	return FShaderValueType::Get(EShaderFundamentalType::Float, 2);
		case EComputeDataInterfaceBufferType::Int3:		return FShaderValueType::Get(EShaderFundamentalType::Int, 3);
		case EComputeDataInterfaceBufferType::Uint3:	return FShaderValueType::Get(EShaderFundamentalType::Uint, 3);
		case EComputeDataInterfaceBufferType::Float3:	return FShaderValueType::Get(EShaderFundamentalType::Float, 3);
		case EComputeDataInterfaceBufferType::Int4:		return FShaderValueType::Get(EShaderFundamentalType::Int, 4);
		case EComputeDataInterfaceBufferType::Uint4:	return FShaderValueType::Get(EShaderFundamentalType::Uint, 4);
		case EComputeDataInterfaceBufferType::Float4:	return FShaderValueType::Get(EShaderFundamentalType::Float, 4);
		case EComputeDataInterfaceBufferType::Int:		
		default:										return FShaderValueType::Get(EShaderFundamentalType::Int);
	}
}

static bool SupportsAtomics(FShaderValueTypeHandle InValueType)
{
	return (InValueType->DimensionType == EShaderFundamentalDimensionType::Scalar && (InValueType->Type == EShaderFundamentalType::Int || InValueType->Type == EShaderFundamentalType::Uint));
}

void UComputeDataInterfaceBuffer::GetSupportedInputs(TArray<FShaderFunctionDefinition>& OutFunctions) const
{
	OutFunctions.AddDefaulted_GetRef()
		.SetName(TEXT("ReadNumValues"))
		.AddReturnType(EShaderFundamentalType::Uint);

	OutFunctions.AddDefaulted_GetRef()
		.SetName(TEXT("ReadValue"))
		.AddReturnType(GetValueTypeFromEnum(ValueType))
		.AddParam(EShaderFundamentalType::Uint);

	OutFunctions.AddDefaulted_GetRef()
		.SetName(TEXT("ReadValueUAV"))
		.AddReturnType(GetValueTypeFromEnum(ValueType))
		.AddParam(EShaderFundamentalType::Uint);
}

void UComputeDataInterfaceBuffer::GetSupportedOutputs(TArray<FShaderFunctionDefinition>& OutFunctions) const
{
	OutFunctions.AddDefaulted_GetRef()
		.SetName(TEXT("WriteValue"))
		.AddParam(EShaderFundamentalType::Uint)
		.AddParam(GetValueTypeFromEnum(ValueType));

	OutFunctions.AddDefaulted_GetRef()
		.SetName(TEXT("WriteAtomicAdd"))
		.AddReturnType(GetValueTypeFromEnum(ValueType))
		.AddParam(EShaderFundamentalType::Uint)
		.AddParam(GetValueTypeFromEnum(ValueType));

	OutFunctions.AddDefaulted_GetRef()
		.SetName(TEXT("WriteAtomicMin"))
		.AddReturnType(GetValueTypeFromEnum(ValueType))
		.AddParam(EShaderFundamentalType::Uint)
		.AddParam(GetValueTypeFromEnum(ValueType));

	OutFunctions.AddDefaulted_GetRef()
		.SetName(TEXT("WriteAtomicMax"))
		.AddReturnType(GetValueTypeFromEnum(ValueType))
		.AddParam(EShaderFundamentalType::Uint)
		.AddParam(GetValueTypeFromEnum(ValueType));
}

BEGIN_SHADER_PARAMETER_STRUCT(FBufferDataInterfaceParameters, )
	SHADER_PARAMETER(uint32, BufferElementCount)
	SHADER_PARAMETER_RDG_BUFFER_SRV(ByteAddressBuffer, BufferSRV)
	SHADER_PARAMETER_RDG_BUFFER_UAV(RWByteAddressBuffer, BufferUAV)
END_SHADER_PARAMETER_STRUCT()

void UComputeDataInterfaceBuffer::GetShaderParameters(TCHAR const* UID, FShaderParametersMetadataBuilder& InOutBuilder, FShaderParametersMetadataAllocations& InOutAllocations) const
{
	InOutBuilder.AddNestedStruct<FBufferDataInterfaceParameters>(UID);
}

TCHAR const* UComputeDataInterfaceBuffer::TemplateFilePath = TEXT("/Plugin/ComputeFramework/Private/ComputeDataInterfaceBuffer.ush");

TCHAR const* UComputeDataInterfaceBuffer::GetShaderVirtualPath() const
{
	return TemplateFilePath;
}

void UComputeDataInterfaceBuffer::GetShaderHash(FString& InOutKey) const
{
	GetShaderFileHash(TemplateFilePath, EShaderPlatform::SP_PCD3D_SM5).AppendString(InOutKey);
}

void UComputeDataInterfaceBuffer::GetHLSL(FString& OutHLSL, FString const& InDataInterfaceName) const
{
	const FString ValueTypeName = GetValueTypeFromEnum(ValueType)->ToString();
	const int32 ValueTypeStride = GetValueTypeFromEnum(ValueType)->GetResourceElementSize();
	const bool bSupportAtomic = SupportsAtomics(GetValueTypeFromEnum(ValueType));

	TMap<FString, FStringFormatArg> TemplateArgs =
	{
		{ TEXT("DataInterfaceName"), InDataInterfaceName },
		{ TEXT("ValueType"), ValueTypeName },
		{ TEXT("ValueTypeStride"), ValueTypeStride },
		{ TEXT("SupportAtomic"), bSupportAtomic ? 1 : 0 },
		{ TEXT("SplitReadWrite"), bAllowReadWrite ? 0 : 1 },
	};

	FString TemplateFile;
	LoadShaderSourceFile(TemplateFilePath, EShaderPlatform::SP_PCD3D_SM5, &TemplateFile, nullptr);
	OutHLSL += FString::Format(*TemplateFile, TemplateArgs);
}

UComputeDataProvider* UComputeDataInterfaceBuffer::CreateDataProvider() const
{
	UBufferDataProvider* Provider = NewObject<UBufferDataProvider>();
	Provider->ValueType = ValueType;
	return Provider;
}

void UBufferDataProvider::Initialize(int32 InDataInterfaceIndex, UComputeDataInterface const* InDataInterface, UObject* InBinding, uint64 InInputMask, uint64 InOutputMask)
{
	if (UComputeGraphComponent* BindingComponent = Cast<UComputeGraphComponent>(InBinding))
	{
		BindingComponent->InitializeProvider(InDataInterfaceIndex, this);
	}

	// Override clear before use if we read but don't write the buffer.
	if (InInputMask != 0 && InOutputMask == 0)
	{
		bClearBeforeUse = true;
	}
}

FComputeDataProviderRenderProxy* UBufferDataProvider::GetRenderProxy()
{
	const uint32 ElementStride = GetValueTypeFromEnum(ValueType)->GetResourceElementSize();
	return new FBufferDataProviderProxy(ElementStride, (uint32)FMath::Max(ElementCount, 0), bClearBeforeUse);
}

FBufferDataProviderProxy::FBufferDataProviderProxy(uint32 InElementStride, uint32 InElementCount, bool bInClearBeforeUse)
	: ElementStride(InElementStride)
	, ElementCount(InElementCount)
	, bClearBeforeUse(bInClearBeforeUse)
{
}

bool FBufferDataProviderProxy::IsValid(FValidationData const& InValidationData) const
{
	if (InValidationData.ParameterStructSize != sizeof(FParameters))
	{
		return false;
	}
	
	if (ElementStride % 4 != 0)
	{
		return false;
	}

	if (ElementCount == 0)
	{
		return false;
	}

	return true;
}

void FBufferDataProviderProxy::AllocateResources(FRDGBuilder& GraphBuilder, FAllocationData const& InAllocationData)
{
	Buffer = GraphBuilder.CreateBuffer(FRDGBufferDesc::CreateByteAddressDesc(ElementStride * ElementCount), TEXT("ComputeFrameworkBuffer"), ERDGBufferFlags::None);
	BufferSRV = GraphBuilder.CreateSRV(Buffer);
	BufferUAV = GraphBuilder.CreateUAV(Buffer, ERDGUnorderedAccessViewFlags::SkipBarrier);

	if (bClearBeforeUse)
	{
		AddClearUAVPass(GraphBuilder, BufferUAV, 0);
	}
}

void FBufferDataProviderProxy::GatherDispatchData(FDispatchData const& InDispatchData)
{
	TStridedView<FParameters> ParameterArray = MakeStridedParameterView<FParameters>(InDispatchData);
	for (int32 InvocationIndex = 0; InvocationIndex < ParameterArray.Num(); ++InvocationIndex)
	{
		FParameters& Parameters = ParameterArray[InvocationIndex];
		Parameters.BufferElementCount = ElementCount;
		Parameters.BufferSRV = BufferSRV;
		Parameters.BufferUAV = BufferUAV;
	}
}
