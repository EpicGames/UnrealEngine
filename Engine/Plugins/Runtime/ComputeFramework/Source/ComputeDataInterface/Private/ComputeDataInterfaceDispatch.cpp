// Copyright Epic Games, Inc. All Rights Reserved.

#include "ComputeFramework/ComputeDataInterfaceDispatch.h"

#include "ComputeFramework/ComputeGraphComponent.h"
#include "ComputeFramework/ShaderParamTypeDefinition.h"
#include "RenderGraphBuilder.h"
#include "ShaderCompilerCore.h"
#include "ShaderParameterMetadataBuilder.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ComputeDataInterfaceDispatch)

void UComputeDataInterfaceDispatch::GetSupportedInputs(TArray<FShaderFunctionDefinition>& OutFunctions) const
{
	OutFunctions.AddDefaulted_GetRef()
		.SetName(TEXT("ReadNumThreads"))
		.AddReturnType(FShaderValueType::Get(EShaderFundamentalType::Uint, 3));
}

BEGIN_SHADER_PARAMETER_STRUCT(FDataInterfaceDispatchParameters, )
	SHADER_PARAMETER(FUintVector3, NumThreads)
END_SHADER_PARAMETER_STRUCT()

void UComputeDataInterfaceDispatch::GetShaderParameters(TCHAR const* UID, FShaderParametersMetadataBuilder& InOutBuilder, FShaderParametersMetadataAllocations& InOutAllocations) const
{
	InOutBuilder.AddNestedStruct<FDataInterfaceDispatchParameters>(UID);
}

const TCHAR* UComputeDataInterfaceDispatch::TemplateFilePath = TEXT("/Plugin/ComputeFramework/Private/ComputeDataInterfaceDispatch.ush");

const TCHAR* UComputeDataInterfaceDispatch::GetShaderVirtualPath() const
{
	return TemplateFilePath;
}

void UComputeDataInterfaceDispatch::GetShaderHash(FString& InOutKey) const
{
	GetShaderFileHash(TemplateFilePath, EShaderPlatform::SP_PCD3D_SM5).AppendString(InOutKey);
}

void UComputeDataInterfaceDispatch::GetHLSL(FString& OutHLSL, FString const& InDataInterfaceName) const
{
	TMap<FString, FStringFormatArg> TemplateArgs =
	{
		{ TEXT("DataInterfaceName"), InDataInterfaceName },
	};

	FString TemplateFile;
	LoadShaderSourceFile(TemplateFilePath, EShaderPlatform::SP_PCD3D_SM5, &TemplateFile, nullptr);
	OutHLSL += FString::Format(*TemplateFile, TemplateArgs);
}

UComputeDataProvider* UComputeDataInterfaceDispatch::CreateDataProvider() const
{
	return NewObject<UDispatchDataProvider>();
}

void UDispatchDataProvider::Initialize(int32 InDataInterfaceIndex, UComputeDataInterface const* InDataInterface, UObject* InBinding, uint64 InInputMask, uint64 InOutputMask)
{
	if (UComputeGraphComponent* BindingComponent = Cast<UComputeGraphComponent>(InBinding))
	{
		BindingComponent->InitializeProvider(InDataInterfaceIndex, this);
	}
}

FComputeDataProviderRenderProxy* UDispatchDataProvider::GetRenderProxy()
{
	FDispatchDataProviderProxy* Proxy = new FDispatchDataProviderProxy(ThreadCount);
	return Proxy;
}

FDispatchDataProviderProxy::FDispatchDataProviderProxy(FIntVector InThreadCount)
	: ThreadCount(InThreadCount)
{
}

bool FDispatchDataProviderProxy::IsValid(FValidationData const& InValidationData) const
{
	if (InValidationData.ParameterStructSize != sizeof(FParameters))
	{
		return false;
	}

	return (ThreadCount.X > 0 && ThreadCount.Y > 0 && ThreadCount.Z > 0);
}

int32 FDispatchDataProviderProxy::GetDispatchThreadCount(TArray<FIntVector>& InOutThreadCounts) const
{
	InOutThreadCounts.Reset();
	InOutThreadCounts.Add(ThreadCount);
	return 1;
}

void FDispatchDataProviderProxy::GatherDispatchData(FDispatchData const& InDispatchData)
{
	const TStridedView<FParameters> ParameterArray = MakeStridedParameterView<FParameters>(InDispatchData);
	for (int32 InvocationIndex = 0; InvocationIndex < ParameterArray.Num(); ++InvocationIndex)
	{
		FParameters& Parameters = ParameterArray[InvocationIndex];
		Parameters.NumThreads = FUintVector(ThreadCount.X, ThreadCount.Y, ThreadCount.Z);
	}
}
