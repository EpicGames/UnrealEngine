// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
    public class ComputeFramework : ModuleRules
    {
        public ComputeFramework(ReadOnlyTargetRules Target) : base(Target)
        {
			PublicDependencyModuleNames.AddRange(
				new string[]
				{
					"RHI", // We include RHIDefinitions.h in a public header, which is required for the symbol HAS_GPU_STATS
				}
			);

            PrivateDependencyModuleNames.AddRange(
                new string[]
                {
                    "Core",
					"CoreUObject",
					"Engine",
					"Projects",
					"RenderCore",
					"Renderer",
				}
			);

			PrivateIncludePathModuleNames.AddRange(
				new string[]
				{
					"DerivedDataCache",
				}
			);
		}
	}
}
