// Copyright Epic Games, Inc. All Rights Reserved.

#include "ComputeFramework/ComputeDataProvider.h"

#include "ComputeWorkerInterface.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ComputeDataProvider)

void FComputeDataProviderRenderProxy::PreSubmit(FRDGBuilder& InGraphBuilder) const
{
	FComputeContext Context = { .GraphBuilder = InGraphBuilder };
	PreSubmit(Context);
}

void FComputeDataProviderRenderProxy::PostSubmit(FRDGBuilder& InGraphBuilder) const
{
	FComputeContext Context = { .GraphBuilder = InGraphBuilder };
	PostSubmit(Context);
}
