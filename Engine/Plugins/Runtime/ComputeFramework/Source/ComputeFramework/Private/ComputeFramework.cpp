// Copyright Epic Games, Inc. All Rights Reserved.

#include "ComputeFramework/ComputeFramework.h"

#include "ComputeFramework/ComputeFrameworkModule.h"
#include "ComputeFramework/ComputeGraph.h"
#include "ComputeFramework/ComputeGraphWorker.h"
#include "ComputeFramework/ComputeKernelShaderCompilationManager.h"
#include "ComputeFramework/ComputeSystem.h"
#include "DataDrivenShaderPlatformInfo.h"
#include "RenderGraphBuilder.h"
#include "ShaderCore.h"
#include "UObject/UObjectIterator.h"
#include "Rendering/RenderCommandPipes.h"

DEFINE_LOG_CATEGORY(LogComputeFramework);

static int32 GComputeFrameworkEnable = 1;
static FAutoConsoleVariableRef CVarComputeFrameworkEnable(
	TEXT("r.ComputeFramework.Enable"),
	GComputeFrameworkEnable,
	TEXT("Enable the Compute Framework.\n"),
	ECVF_Scalability | ECVF_RenderThreadSafe
	);

static int32 GComputeFrameworkDeferredCompilation = 1;
static FAutoConsoleVariableRef CVarComputeFrameworkDeferredCompilation(
	TEXT("r.ComputeFramework.DeferredCompilation"),
	GComputeFrameworkDeferredCompilation,
	TEXT("Compile compute graphs on first usage instead of on PostLoad().\n"),
	ECVF_Default
);

FAutoConsoleCommand CmdRebuildComputeGraphs(
	TEXT("r.ComputeFramework.RebuildComputeGraphs"),
	TEXT("Force all loaded UComputeGraph objects to rebuild."),
	FConsoleCommandDelegate::CreateStatic(ComputeFramework::RebuildComputeGraphs)
);

namespace ComputeFramework
{
	bool IsSupported(EShaderPlatform ShaderPlatform)
	{
		return FDataDrivenShaderPlatformInfo::GetSupportsComputeFramework(ShaderPlatform);
	}

	bool IsEnabled()
	{
		return GComputeFrameworkEnable > 0 && FComputeFrameworkModule::GetComputeSystem() != nullptr;
	}

	bool IsDeferredCompilation()
	{
#if WITH_EDITOR
		return
			GComputeFrameworkDeferredCompilation != 0 &&
			!(IsRunningCommandlet() && IsAllowCommandletRendering()) &&
			!GIsAutomationTesting; // Making sure shader is ready before running the test, this helps avoid sync compilation during SendAllEndOfFrameUpdates, which is not allowed
#else
		return false;
#endif
	}

	void RebuildComputeGraphs()
	{
#if WITH_EDITOR
		FlushShaderFileCache();

		for (TObjectIterator<UComputeGraph> It; It; ++It)
		{
			It->UpdateResources();
		}
#endif // WITH_EDITOR
	}

	void TickCompilation(float DeltaSeconds)
	{
#if WITH_EDITOR
		GComputeKernelShaderCompilationManager.Tick(DeltaSeconds);
#endif // WITH_EDITOR
	}

	void FlushWork(FSceneInterface const* InScene, FName InExecutionGroupName)
	{
		FComputeFrameworkSystem* ComputeSystem = FComputeFrameworkModule::GetComputeSystem();
		FComputeGraphTaskWorker* ComputeGraphWorker = ComputeSystem != nullptr ? ComputeSystem->GetComputeWorker(InScene) : nullptr;
		if (ComputeGraphWorker)
		{
			UE::RenderCommandPipe::FSyncScope SyncScope({ &UE::RenderCommandPipe::SkeletalMesh });

			ENQUEUE_RENDER_COMMAND(ComputeFrameworkFlushCommand)(
				[ComputeGraphWorker, InExecutionGroupName, InScene](FRHICommandListImmediate& RHICmdList)
				{
					FRDGBuilder GraphBuilder(RHICmdList);

					FComputeContext Context
					{
						.GraphBuilder = GraphBuilder,
						.ExecutionGroupName = InExecutionGroupName,
						.FeatureLevel = GMaxRHIFeatureLevel,
						.Scene = InScene,
					};

					ComputeGraphWorker->SubmitWork(Context);

					GraphBuilder.Execute();
				});
		}
	}

	void AbortWork(FSceneInterface const* InScene, UObject* InOwnerPointer)
	{
		FComputeFrameworkSystem* ComputeSystem = FComputeFrameworkModule::GetComputeSystem();
		FComputeGraphTaskWorker* ComputeGraphWorker = ComputeSystem != nullptr ? ComputeSystem->GetComputeWorker(InScene) : nullptr;
		if (ComputeGraphWorker)
		{
			ENQUEUE_RENDER_COMMAND(ComputeFrameworkAbortCommand)(
				[ComputeGraphWorker, InOwnerPointer](FRHICommandListImmediate& RHICmdList)
			{
				ComputeGraphWorker->Abort(InOwnerPointer);
			});
		}
	}
}
