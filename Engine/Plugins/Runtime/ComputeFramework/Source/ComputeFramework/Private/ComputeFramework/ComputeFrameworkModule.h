// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ComputeFramework/IComputeFrameworkModule.h"

class FComputeFrameworkSystem;

/** ComputeFramework module implementation. */
class FComputeFrameworkModule : public IComputeFrameworkModule
{
public:
	/** IModuleInterface implementation */
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;

	/** Get the instantiation of the compute system associated with the module. */
	static FComputeFrameworkSystem* GetComputeSystem() { return ComputeSystem; }

private:
	static FComputeFrameworkSystem* ComputeSystem;
};
