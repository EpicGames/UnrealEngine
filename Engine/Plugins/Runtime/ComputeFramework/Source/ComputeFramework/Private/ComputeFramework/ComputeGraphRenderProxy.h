// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ComputeFramework/ComputeKernelPermutationVector.h"
#include "CoreMinimal.h"

struct FComputeKernelPermutationVector;
class FComputeKernelResource;
class FShaderParametersMetadata;
struct FShaderParametersMetadataAllocations;

/** 
 * Render thread proxy object for a UComputeGraph. 
 * Owns a self contained copy of everything that needs to be read from the render thread.
 */
class FComputeGraphRenderProxy
{
public:
	/** Description for each kernel in the graph. */
	struct FKernelInvocation
	{
		/** Friendly kernel name. */
		FString KernelName;
		/** Group thread size for kernel. */
		FIntVector KernelGroupSize = FIntVector(1, 1, 1);
		/** Kernel resource object. Owned by the UComputeGraph but contains render thread safe accessible shader map. */
		FComputeKernelResource const* KernelResource = nullptr;
		/** Shader parameter metadata. */
		FShaderParametersMetadata* ShaderParameterMetadata = nullptr;
		/** Array of indices into the full graph data provider array. Contains only the indices to data providers that this kernel references. */
		TArray<int32> BoundProviderIndices;
		/** Array of indices for data providers that should trigger a readback. */
		TArray<int32> ReadbackProviderIndices;
		/** Indices of data providers that require PreSubmit call. */
		TArray<int32> PreSubmitProviderIndices;
		/** Indices of data providers that require PostSubmit call. */
		TArray<int32> PostSubmitProviderIndices;
		/** Same size as BoundProviderIndices, non-primary data providers will be forced to present a full view to its data (unified)*/
		TArray<bool> BoundProviderIsPrimary;
		/** The index of the special execution data provider in the full graph data provider array. */
		int32 ExecutionProviderIndex = -1;
		/** Whether the kernel can combine multiple supports combined sub-invocations. */
		bool bSupportsUnifiedDispatch = false;
		/** Whether a render capture should be triggered in the scope around this kernel invocation. */
		bool bTriggerRenderCapture = false;
	};

	/** Friendly name for the owner graph. */
	FName GraphName;
	/** Kernel invocation information per kernel. */
	TArray<FKernelInvocation> KernelInvocations;
	/** Shader permutations vector per kernel. */
	TArray<FComputeKernelPermutationVector> ShaderPermutationVectors;
	/** Container for allocations from the building of all of the kernel FShaderParametersMetadata objects. */
	TUniquePtr<FShaderParametersMetadataAllocations> ShaderParameterMetadataAllocations;
};
