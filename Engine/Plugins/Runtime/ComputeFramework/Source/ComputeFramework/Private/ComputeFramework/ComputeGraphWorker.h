// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Async/Future.h"
#include "ComputeFramework/ComputeDataProvider.h"
#include "ComputeFramework/ComputeFrameworkKernelTiming.h"
#include "ComputeWorkerInterface.h"
#include "RHIDefinitions.h"
#include "Shader.h"

class FComputeDataProviderRenderProxy;
class FComputeGraphRenderProxy;
class FComputeKernelResource;
class FComputeKernelShader;

/** 
 * Class that manages the scheduling of Compute Graph work.
 * Work can be enqueued on the render thread for the execution at the next call to ExecuteBatches().
 */
class FComputeGraphTaskWorker : public IComputeTaskWorker
{
public:
	/** Enqueue a compute graph for execution. */
	void Enqueue(
		FName InExecutionGroupName,
		FName InOwnerName,
		uint8 InGraphSortPriority,
		FComputeGraphRenderProxy const* InGraphRenderProxy, 
		TArray<FComputeDataProviderRenderProxy*> InDataProviderRenderProxies,
		FSimpleDelegate InFallbackDelegate,
		bool bInEnableRenderCaptures,
#if HAS_GPU_STATS
		TArray<TPromise<FComputeFrameworkKernelTiming>> InKernelTimingPromises,
#endif
		const UObject* OwnerPointer = nullptr);

	void Abort(const UObject* OwnerPointer);

	/** Has enqueued compute graph work. */
	bool HasWork(FName InExecutionGroupName) const override;

	/** Submit enqueued compute graph work. */
	void SubmitWork(FComputeContext& Context) override;

private:
	/** Poll active readbacks and execute callbacks for ready ones. */
	void UpdateReadbacks();

private:
	/** Description of each graph that is enqueued. */
	struct FGraphInvocation
	{
		/** Name of owner object that invoked the graph. */
		FName OwnerName;
		/** Pointer to an owning UObject. */
		const UObject* OwnerPointer = nullptr;
		/** Priority used when sorting work. */
		uint8 GraphSortPriority = 0;
		/** Graph render proxy. */
		FComputeGraphRenderProxy const* GraphRenderProxy = nullptr;
		/** Data provider render proxies. */
		TArray<FComputeDataProviderRenderProxy*> DataProviderRenderProxies;
		/** Render thread fallback logic for invocations that are invalid. */
		FSimpleDelegate FallbackDelegate;
		/** Whether render captures are enabled. */
		bool bEnableRenderCaptures = false;
#if HAS_GPU_STATS
		/** One TPromise per kernel (parallel to graph kernels). Moved into FComputeFrameworkGPUStat objects during SubmitWork. */
		mutable TArray<TPromise<FComputeFrameworkKernelTiming>> KernelTimingPromises;

		/** Fulfills any remaining promises with a zero result so consumers don't block, then clears the array.
		 *  Called on validation failure (!bIsValid path) and explicitly from Abort() before the invocation is removed. */
		void SendNullKernelTimings() const;
#endif

		~FGraphInvocation();
	};

	/** Map of enqueued work per execution group . */
	TMap<FName, TArray<FGraphInvocation> > GraphInvocationsPerGroup;

	/** Description of a single dispatch group to be submitted. */
	struct FSubmitDescription
	{
		/**
		 * Sort key allows us to sort dispatches for optimum scheduling.
		 * Syncing is usually required between consecutive kernels in a graph.
		 * So we schedule the first kernels of all the graphs, before all of the second kernels.
		 * That reduces time sync time, at the expense of memory pressure for buffers that need to stay alive.
		 * In future we may want to add a limit to the number of graphs in flight to avoid memory pressure.
		 */
		union
		{
			uint32 PackedSortKey = 0;
			struct
			{
				uint32 GraphIndex : 12;			// Graph index.
				uint32 KernelIndex : 12;		// Kernel index within the graph.
				uint32 GraphSortPriority : 8;	// Externally defined sort priority to maintain inter-graph dependencies.
			};
		};

		/** Track the index into our collected shader array. */
		uint32 ShaderIndex : 15;
		/** Track if this is Unified Dispatch. */
		uint32 bIsUnified : 1;
		/** Whether a render capture should be triggered in the scope around this submit. */
		uint32 bTriggerRenderCapture : 1;
	};

	// These arrays could be local to FComputeGraphTaskWorker::SubmitWork() but we 
	// store them with class and Reset() them at each usage to avoid per frame array allocations.
	TArray<FSubmitDescription> SubmitDescs;
	TArray<TShaderRef<FComputeKernelShader>> Shaders;
	TArray<int32> PermutationIds;
	TArray<FIntVector> ThreadCounts;
	TArray<FComputeDataProviderRenderProxy::FReadbackData> ReadbackDatas;

	/** Data for tracking active asynchronous readbacks. */
	struct FAsyncReadback
	{
		~FAsyncReadback();

		class FRHIGPUBufferReadback* Readback = nullptr;
		uint32 NumBytes = 0;
		const UObject* OwnerPointer = nullptr;
		FComputeDataProviderRenderProxy::FReadbackCallback OnDataAvailable;
	};

	TArray<FAsyncReadback> ActiveAsyncReadbacks;
};
