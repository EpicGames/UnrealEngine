// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

// In case of merge conflicts with DDC versions, you MUST generate a new GUID and set this new guid as version
#define COMPUTEKERNEL_DERIVEDDATA_VER	TEXT("F4E7148D-2146-4CFA-B831-B8DB48382101")
