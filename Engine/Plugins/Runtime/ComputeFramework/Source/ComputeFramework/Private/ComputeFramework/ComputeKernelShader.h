// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GlobalShader.h"
#include "ComputeKernelShaderType.h"
#include "ComputeKernelShared.h"
#include "SceneView.h"
#include "Shader.h"
#include "ShaderParameters.h"

class FTextureResource;
class UClass;

class FComputeKernelShader : public FShader
{
	DECLARE_SHADER_TYPE(FComputeKernelShader, ComputeKernel);

public:
	FComputeKernelShader() = default;
	FComputeKernelShader(
		const FComputeKernelShaderType::CompiledShaderInitializerType & Initializer
		);

	MS_ALIGN(SHADER_PARAMETER_STRUCT_ALIGNMENT) struct FParameters
	{
	};
};
