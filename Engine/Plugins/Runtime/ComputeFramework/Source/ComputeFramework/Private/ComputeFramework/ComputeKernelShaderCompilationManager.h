// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ComputeKernelShared.h"
#include "ShaderCompiler.h"

/** Results for a single compiled shader map. */
struct FComputeKernelShaderMapCompileResults
{
	FComputeKernelShaderMapCompileResults() :
		NumJobsQueued(0),
		bAllJobsSucceeded(true),
		bRecreateComponentRenderStateOnCompletion(false)
	{}

	int32 NumJobsQueued;
	bool bAllJobsSucceeded;
	bool bRecreateComponentRenderStateOnCompletion;
	TArray<FShaderCommonCompileJobPtr> FinishedJobs;
};

/** Results for a single compiled and finalized shader map. */
struct FComputeKernelShaderMapFinalizeResults : public FComputeKernelShaderMapCompileResults
{
	/** Tracks finalization progress on this shader map. */
	int32 FinalizeJobIndex;

	FComputeKernelShaderMapFinalizeResults(const FComputeKernelShaderMapCompileResults& InCompileResults) :
		FComputeKernelShaderMapCompileResults(InCompileResults),
		FinalizeJobIndex(0)
	{}
};

#if WITH_EDITOR

/** Handles finished shader compile jobs, applying of the shaders to their config asset, and some error handling. */
class FComputeKernelShaderCompilationManager
{
public:
	void Tick(float DeltaSeconds);
	void AddJobs(TArray<FShaderCommonCompileJobPtr> InNewJobs);
	void FinishCompilation(const TCHAR* InKernelName, const TArray<int32>& ShaderMapIdsToFinishCompiling);
	void CancelCompilation(const TCHAR* InKernelName, const TArray<int32>& ShaderMapIdsToCancelCompiling);

private:
	void ProcessAsyncResults();
	void ProcessCompiledComputeKernelShaderMaps(TMap<int32, FComputeKernelShaderMapFinalizeResults>& CompiledShaderMaps, float TimeBudget);

	TArray<FShaderCommonCompileJobPtr> JobQueue;

	/** Map from shader map Id to the compile results for that map, used to gather compiled results. */
	TMap<int32, FComputeKernelShaderMapCompileResults> ComputeKernelShaderMapJobs;

	/** Map from shader map id to results being finalized.  Used to track shader finalizations over multiple frames. */
	TMap<int32, FComputeKernelShaderMapFinalizeResults> PendingFinalizeComputeKernelShaderMaps;
};

extern FComputeKernelShaderCompilationManager GComputeKernelShaderCompilationManager;

#endif // WITH_EDITOR
