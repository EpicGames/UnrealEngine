// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Engine/EngineTypes.h"
#include "GlobalShader.h"
#include "Shader.h"
#if WITH_EDITOR
#include "ShaderCompiler.h"
#endif

class FShaderCommonCompileJob;
class FShaderCompileJob;
class FComputeKernelResource;


struct FComputeKernelShaderPermutationParameters : public FShaderPermutationParameters
{
	FComputeKernelShaderPermutationParameters(EShaderPlatform InPlatform)
		: FShaderPermutationParameters(InPlatform)
	{
	}
};

class FComputeKernelShaderType : public FShaderType
{
public:
	struct FParameters : public FShaderType::FParameters
	{
		const FShaderParametersMetadata& ShaderParamMetadata;

		FParameters(
			const FShaderParametersMetadata& InShaderParamMetadata
			)
			: ShaderParamMetadata(InShaderParamMetadata)
		{
		}
	};

	struct CompiledShaderInitializerType : FGlobalShaderType::CompiledShaderInitializerType
	{
		const FString DebugDescription;

		CompiledShaderInitializerType(
			const FShaderType* InType,
			const FParameters* InParameters,
			int32 InPermutationId,
			const FShaderCompilerOutput& CompilerOutput,
			uint64 InComputeKernelShaderMapHash,
			const FString& InDebugDescription
			)
			: FGlobalShaderType::CompiledShaderInitializerType(
				InType,
				InParameters,
				InPermutationId,
				CompilerOutput, 
				InComputeKernelShaderMapHash,
				nullptr,
				nullptr
				)
			, DebugDescription(InDebugDescription)
		{
		}
	};

	FComputeKernelShaderType(
		FTypeLayoutDesc& InTypeLayout,
		const TCHAR* InName,
		const TCHAR* InSourceFilename,
		const TCHAR* InFunctionName,
		uint32 InFrequency,					// ugly - ignored for but needed for IMPLEMENT_SHADER_TYPE macro magic
		int32 InTotalPermutationCount,
		int32 InTotalSpecializationCount,
		ConstructSerializedType InConstructSerializedRef,
		ConstructCompiledType InConstructCompiledRef,
		ShouldCompilePermutationType InShouldCompilePermutationRef,
		ShouldPrecachePermutationType InShouldPrecachePermutationRef,
		GetUnspecializedIdType InGetUnspecializedIdRef,
		GetSpecializationValuesType InGetSpecializationValuesRef,
		GetRayTracingPayloadTypeType InGetRayTracingPayloadTypeRef,
		GetShaderBindingLayoutType InGetShaderBindingLayoutTypeRef,
#if WITH_EDITOR
		ModifyCompilationEnvironmentType InModifyCompilationEnvironmentRef,
		ValidateCompiledResultType InValidateCompiledResultRef,
		GetOverrideJobPriorityType InGetOverrideJobPriorityRef,
#endif // WITH_EDITOR
		uint32 InTypeSize,
		const FShaderParametersMetadata* InRootParametersMetadata = nullptr
#if WITH_EDITOR
		, GetPermutationIdStringType InGetPermutationIdStringRef = nullptr
#endif // WITH_EDITOR
		):
		FShaderType(
			EShaderTypeForDynamicCast::ComputeKernel, 
			InTypeLayout, 
			InName, 
			InSourceFilename, 
			InFunctionName, 
			SF_Compute, 
			InTotalPermutationCount,
			InTotalSpecializationCount,
			InConstructSerializedRef,
			InConstructCompiledRef,
			InShouldCompilePermutationRef,
			InShouldPrecachePermutationRef,
			InGetUnspecializedIdRef,
			InGetSpecializationValuesRef,
			InGetRayTracingPayloadTypeRef,
			InGetShaderBindingLayoutTypeRef,
#if WITH_EDITOR
			InModifyCompilationEnvironmentRef,
			InValidateCompiledResultRef,
			InGetOverrideJobPriorityRef,
#endif // WITH_EDITOR
			InTypeSize,
			InRootParametersMetadata
#if WITH_EDITOR
			, InGetPermutationIdStringRef
#endif // WITH_EDITOR
			)
	{
	}

#if WITH_EDITOR
	void BeginCompileShader(
		uint32 ShaderMapId,
		int32 PermutationId,
		const FComputeKernelResource* InKernel,
		FSharedShaderCompilerEnvironment* InCompilationEnvironment,
		EShaderPlatform Platform,
		TArray<FShaderCommonCompileJobPtr>& InOutNewJobs,
		FShaderTarget Target
		);

	/**
	 * Either creates a new instance of this type or returns an equivalent existing shader.
	 */
	FShader* FinishCompileShader(
		uint64 InComputeKernelShaderMapHash,
		const FShaderCompileJob& CurrentJob,
		const FString& InDebugDescription
		) const;
#endif // WITH_EDITOR

	/**
	 * Checks if the shader type should be cached.
	 * @return True if this shader type should be cached.
	 */
	bool ShouldCache(EShaderPlatform InPlatform, const FComputeKernelResource* Kernel) const
	{
		return ShouldCompilePermutation(FComputeKernelShaderPermutationParameters(InPlatform));
	}

#if WITH_EDITOR
protected:
	/**
	 * Sets up the environment used to compile an instance of this shader type.
	 * @param InPlatform - Platform to compile for.
	 * @param OutEnvironment - The shader compile environment that the function modifies.
	 */
	void SetupCompileEnvironment(EShaderPlatform InPlatform, const FComputeKernelResource* Kernel, FShaderCompilerEnvironment& OutEnvironment) const
	{
		ModifyCompilationEnvironment(FComputeKernelShaderPermutationParameters(InPlatform), OutEnvironment);
	}
#endif // WITH_EDITOR
};
