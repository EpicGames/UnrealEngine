// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "ComputeFramework/ComputeKernelCompileResult.h"
#include "Engine/EngineTypes.h"
#include "RenderResource.h"
#include "RenderingThread.h"
#include "RenderDeferredCleanup.h"
#include "RHI.h"
#include "SceneTypes.h"
#include "Shader.h"
#include "ShaderCompiler.h"
#include "Templates/RefCounting.h"

struct FComputeKernelDefinitionSet;
struct FComputeKernelPermutationVector;
class FComputeKernelResource;
class FComputeKernelShaderMap;
class FComputeKernelShader;
class FComputeKernelShaderMapId;
class UComputeKernelSource;
class FShaderParametersMetadata;
struct FShaderParametersMetadataAllocations;

/** Stores outputs from the  kernel compile that need to be saved. */
class FComputeKernelCompilationOutput
{
	DECLARE_TYPE_LAYOUT(FComputeKernelCompilationOutput, NonVirtual);
public:
	FComputeKernelCompilationOutput()
	{}
};

/** Contains all the information needed to uniquely identify a FComputeKernelShaderMapID. */
class FComputeKernelShaderMapId
{
	DECLARE_TYPE_LAYOUT(FComputeKernelShaderMapId, NonVirtual);
public:
	/** Feature level that the shader map is going to be compiled for.  */
	LAYOUT_FIELD(ERHIFeatureLevel::Type, FeatureLevel);

	/**
	 * The hash computed from the shader code generated by the library.
	 */
	LAYOUT_FIELD(uint64, ShaderCodeHash);
	
	/** Shader types of shaders that are inlined in this shader map in the DDC. */
	LAYOUT_FIELD(TMemoryImageArray<FShaderTypeDependency>, ShaderTypeDependencies);

	/*
	* Type layout parameters of the memory image
	*/
	LAYOUT_FIELD(FPlatformTypeLayoutParameters, LayoutParams);

public:
	FComputeKernelShaderMapId()
		: FeatureLevel(ERHIFeatureLevel::SM5)
	{ }

	~FComputeKernelShaderMapId() = default;

#if WITH_EDITOR
	void SetShaderDependencies(const TArray<FShaderType*>& InShaderTypes, EShaderPlatform InShaderPlatform);
#endif

	//void Serialize(FArchive& Ar);

	friend uint32 GetTypeHash(const FComputeKernelShaderMapId& Ref)
	{
		return GetTypeHash(Ref.ShaderCodeHash);
	}

	SIZE_T GetSizeBytes() const
	{
		return sizeof(*this) + ShaderTypeDependencies.GetAllocatedSize();
	}

	/** Hashes the kernel specific part of this shader map Id. */
	uint64 GetComputeKernelHash() const;

	/**
	 * Tests this set against another for equality, disregarding override settings.
	 *
	 * @param InReferenceSet	The set to compare against
	 * @return					true if the sets are equal
	 */
	bool operator==(const FComputeKernelShaderMapId& InReferenceSet) const;

	bool operator!=(const FComputeKernelShaderMapId& InReferenceSet) const
	{
		return !(*this == InReferenceSet);
	}

#if WITH_EDITOR
	/** Appends string representations of this Id to a key string. */
	void AppendKeyString(FString& OutKeyString) const;
#endif // WITH_EDITOR

	/** Returns true if the requested shader type is a dependency of this shader map Id. */
	bool ContainsShaderType(const FShaderType* ShaderType) const;
};

class FComputeKernelShaderMapContent : public FShaderMapContent
{
	using Super = FShaderMapContent;
	friend class FComputeKernelShaderMap;
	DECLARE_TYPE_LAYOUT(FComputeKernelShaderMapContent, NonVirtual);
private:
	explicit FComputeKernelShaderMapContent(EShaderPlatform InPlatform) : Super(InPlatform) {}

	/** The kernel's user friendly name */
	LAYOUT_FIELD(FMemoryImageString, FriendlyName);

	/** The static parameter set that this shader map was compiled with */
	LAYOUT_FIELD(FComputeKernelShaderMapId, ShaderMapId);

	/** Shader compilation output */
	LAYOUT_FIELD(FComputeKernelCompilationOutput, CompilationOutput);
};


/**
 * All the shader permutations for a kernel.
 */
class FComputeKernelShaderMap : public TShaderMap<FComputeKernelShaderMapContent, FShaderMapPointerTable>, public FDeferredCleanupInterface
{
public:
	using Super = TShaderMap<FComputeKernelShaderMapContent, FShaderMapPointerTable>;

	/**
	 * Finds the shader map for a kernel.
	 * @param InShaderMapId - Id to find
	 * @param Platform - The platform to lookup for
	 * @return nullptr if no cached shader map was found.
	 */
	static TRefCountPtr<FComputeKernelShaderMap> FindId(const FComputeKernelShaderMapId& InShaderMapId, EShaderPlatform InPlatform);

	// ShaderMap interface
	template<typename ShaderType> TShaderRef<ShaderType> GetShader(int32 PermutationId) const { return TShaderRef<ShaderType>(GetContent()->GetShader<ShaderType>(PermutationId), *this); }
	TShaderRef<FShader> GetShader(FShaderType* ShaderType, int32 PermutationId) const { return TShaderRef<FShader>(GetContent()->GetShader(ShaderType, PermutationId), *this); }

#if WITH_EDITOR
	/**
	 * Attempts to load the shader map for the given kernel from the Derived Data Cache.
	 * If InOutShaderMap is valid, attempts to load the individual missing shaders instead.
	 */
	static void LoadFromDerivedDataCache(const FComputeKernelResource* InKernel, const FComputeKernelShaderMapId& InShaderMapId, EShaderPlatform InPlatform, TRefCountPtr<FComputeKernelShaderMap>& InOutShaderMap);
#endif // WITH_EDITOR

	FComputeKernelShaderMap();
	~FComputeKernelShaderMap();

#if WITH_EDITOR
	/**
	 * Compiles the shaders for a kernel and caches them in this shader map.
	 * @param InKernel - The kernel to compile shaders for.
	 * @param InShaderMapId - the set of static parameters to compile for
	 * @param InPlatform - The platform to compile to
	 */
	void Compile(
		FComputeKernelResource* InKernel,
		const FComputeKernelShaderMapId& InShaderMapId,
		TRefCountPtr<FSharedShaderCompilerEnvironment> InCompilationEnvironment,
		const FComputeKernelCompilationOutput& InCompilationOutput,
		EShaderPlatform InPlatform,
		bool bSynchronousCompile,
		bool bApplyCompletedShaderMapForRendering
		);

	/** Sorts the incoming compiled jobs into the appropriate shader maps, and finalizes this shader map so that it can be used for rendering. */
	bool ProcessCompilationResults(const TArray<FShaderCommonCompileJobPtr>& InCompilationResults, int32& InOutResultIndex, float& InOutTimeBudget);
#endif // WITH_EDITOR

	/**
	 * Checks whether the shader map is missing any shader types necessary for the given kernel.
	 * @param InKernel - The ComputeKernel which is checked.
	 * @return True if the shader map has all of the shader types necessary.
	 */
	bool IsComplete(const FComputeKernelResource* InKernel, bool bSilent);

#if WITH_EDITOR
	/**
	 * Checks to see if the shader map is already being compiled for another kernel, and if so
	 * adds the specified kernel to the list to be applied to once the compile finishes.
	 * @param InKernel - The ComputeKernel we also wish to apply the compiled shader map to.
	 * @return True if the shader map was being compiled and we added it to the list to be applied.
	 */
	bool TryToAddToExistingCompilationTask(FComputeKernelResource* InKernel);
#endif // WITH_EDITOR

	/** Builds a list of the shaders in a shader map. */
	void GetShaderList(TMap<FShaderId, TShaderRef<FShader>>& OutShaders) const;
	virtual void GetShaderList(TMap<FHashedName, TShaderRef<FShader>>& OutShaders) const override;
	virtual void GetShaderPipelineList(TArray<FShaderPipelineRef>& OutShaderPipelines) const override;

	/** Registers a ComputeKernel shader map in the global map so it can be used. */
	void Register(EShaderPlatform InShaderPlatform);

	// Reference counting.
	void AddRef();
	void Release();

#if WITH_EDITOR
	/** Removes a kernel from ComputeKernelShaderMapsBeingCompiled. */
	static void RemovePending(FComputeKernelResource* InKernel);
#endif

	/** Serializes the shader map. */
	bool Serialize(FArchive& Ar);

#if WITH_EDITOR
	/** Saves this shader map to the derived data cache. */
	void SaveToDerivedDataCache();
#endif

	// Accessors.
	const FComputeKernelShaderMapId& GetShaderMapId() const	{ return GetContent()->ShaderMapId; }
	EShaderPlatform GetShaderPlatform() const				{ return GetContent()->GetShaderPlatform(); }
	const FMemoryImageString& GetFriendlyName() const		{ return GetContent()->FriendlyName; }
	uint32 GetCompilingId() const							{ return CompilingId; }
	bool IsCompilationFinalized() const						{ return bCompilationFinalized; }
	bool CompiledSuccessfully() const						{ return bCompiledSuccessfully; }

	bool IsValid() const
	{
		return bCompilationFinalized && bCompiledSuccessfully && !bDeletedThroughDeferredCleanup;
	}

	int32 GetNumRefs() const { return NumRefs; }
	uint32 GetCompilingId()  { return CompilingId; }

#if WITH_EDITOR
	static TMap<TRefCountPtr<FComputeKernelShaderMap>, TArray<FComputeKernelResource*> > &GetInFlightShaderMaps()
	{
		// ComputeKernelShaderMapsBeingCompiled is game thread only.
		check(IsInGameThread());
		return ComputeKernelShaderMapsBeingCompiled;
	}

	void SetCompiledSuccessfully(bool bSuccess) { bCompiledSuccessfully = bSuccess; }
#endif // WITH_EDITOR

private:

	/**
	 * A global map from a kernel's ID and static switch set to any shader map cached for that kernel.
	 * Note: this does not necessarily contain all kernel shader maps in memory.  Shader maps with the same key can evict each other.
	 * No ref counting needed as these are removed on destruction of the shader map.
	 */
	static TMap<FComputeKernelShaderMapId, FComputeKernelShaderMap*> GIdToComputeKernelShaderMap[SP_NumPlatforms];

	// Protects GIdToComputeKernelShaderMap plus associated state. 
	// These are mutated from any thread that constructs/destroys/registers/unregisters a FComputeKernelShaderMap
	// including async-loading worker threads.
	static FCriticalSection ShaderMapRegistryMutex;

#if WITH_EDITOR
	// Tracks resources and their shader maps that are being compiled. 
	// Game-thread only, asserted by check(IsInGameThread()) at every access point.
	static TMap<TRefCountPtr<FComputeKernelShaderMap>, TArray<FComputeKernelResource*> > ComputeKernelShaderMapsBeingCompiled;
#endif

	/** Uniquely identifies this shader map during compilation, needed for deferred compilation where shaders from multiple shader maps are compiled together. */
	uint32 CompilingId;

	mutable int32 NumRefs;

	/** Used to catch errors where the shader map is deleted directly. */
	std::atomic<bool> bDeletedThroughDeferredCleanup;

	/** Indicates whether this shader map has been registered in GIdToComputeKernelShaderMap */
	std::atomic<bool> bRegistered;

	/**
	 * Indicates whether this shader map has had ProcessCompilationResults called after Compile.
	 * The shader map must not be used on the rendering thread unless bCompilationFinalized is true.
	 */
	std::atomic<bool> bCompilationFinalized;

	std::atomic<bool> bCompiledSuccessfully;

	/** Indicates whether the shader map should be stored in the shader cache. */
	uint32 bIsPersistent : 1;

	uint32 bHasFrozenContent : 1;

#if WITH_EDITOR
	FShader* ProcessCompilationResultsForSingleJob(FShaderCompileJob& InSingleJob, uint64 InShaderMapHash);
#endif

	bool IsComputeKernelShaderComplete(const FComputeKernelResource* InKernel, const FComputeKernelShaderType* InShaderType, bool bSilent);

	friend class FShaderCompilingManager;
};

/** Delegate for kernel compile completion. */
DECLARE_DELEGATE_OneParam(FOnComputeKernelCompilationComplete, FComputeKernelResource const*);

/**
 * FComputeKernelResource represents a UComputeKernel to the shader compilation process
 */
class FComputeKernelResource
{
public:
	FComputeKernelResource();
	virtual ~FComputeKernelResource();

	/**
	 * Caches the shaders for this kernel with no static parameters on the given platform.
	 * This is used by UComputeKernel
	 */
	bool CacheShaders(EShaderPlatform InPlatform, const ITargetPlatform* TargetPlatform, bool bApplyCompletedShaderMapForRendering, bool bSynchronous);
	bool CacheShaders(const FComputeKernelShaderMapId& InShaderMapId, EShaderPlatform InPlatform, bool bApplyCompletedShaderMapForRendering, bool bSynchronous);
	bool CacheInlineShaders(EShaderPlatform InPlatform);

	/**
	 * Should the shader for this kernel with the given platform and shader type combination be compiled
	 *
	 * @param InPlatform		The platform currently being compiled for
	 * @param InShaderType		Which shader is being compiled
	 *
	 * @return true if the shader should be compiled
	 */
	virtual bool ShouldCache(EShaderPlatform InPlatform, const FShaderType* InShaderType) const;

	void SerializeShaderMap(FArchive& Ar);

	/** Releases this kernel's shader map.  Must only be called on ComputeKernels not exposed to the rendering thread! */
	void ReleaseShaderMap();

	/** Discards loaded shader maps if the application can't render */
	void DiscardShaderMap();

	void GetDependentShaderTypes(EShaderPlatform InPlatform, TArray<FShaderType*>& OutShaderTypes) const;
	virtual void GetShaderMapId(EShaderPlatform InPlatform, const ITargetPlatform* TargetPlatform, FComputeKernelShaderMapId& OutId) const;

	void Invalidate();

	/**
	 * Should shaders compiled for this kernel be saved to disk?
	 */
	virtual bool IsPersistent() const { return true; }

	/**
	 * Called when compilation finishes, after the GameThreadShaderMap is set and the render command to set the RenderThreadShaderMap is queued
	 */
	virtual void NotifyCompilationFinished(FString const& ResultMessage);

#if WITH_EDITOR
	/**
	 * Cancels all outstanding compilation jobs
	 */
	void CancelCompilation();

	/**
	 * Blocks until compilation has completed. Returns immediately if a compilation is not outstanding.
	 */
	void FinishCompilation();
#endif // WITH_EDITOR

	/**
	 * Checks if the compilation for this shader is finished
	 *
	 * @return returns true if compilation is complete false otherwise
	 */
	bool IsCompilationFinished() const;

	// Accessors.
	FComputeKernelCompileResults const& GetCompilationResults() const { return CompilationResults; }
	void SetCompilationResults(FComputeKernelCompileResults const& InCompilationResults) { CompilationResults = InCompilationResults; }

	ERHIFeatureLevel::Type GetFeatureLevel() const { return FeatureLevel; }

	FComputeKernelShaderMap* GetGameThreadShaderMap() const
	{
		checkSlow(IsInGameThread() || IsInAsyncLoadingThread() || IsInParallelLoadingThread());
		return GameThreadShaderMap;
	}

	/** Returns owner name for tracking */
	FName GetOwnerFName() const
	{
		return AssetPath;
	}

	/** Note: SetRenderingThreadShaderMap must also be called with the same value, but from the rendering thread. */
	void SetGameThreadShaderMap(FComputeKernelShaderMap* InShaderMap)
	{
		checkSlow(IsInGameThread() || IsInAsyncLoadingThread() || IsInParallelLoadingThread());
		GameThreadShaderMap = InShaderMap;
		if (LIKELY(GameThreadShaderMap))
		{
			GameThreadShaderMap->GetResource()->SetOwnerName(GetOwnerFName());
		}
	}

	/** Note: SetGameThreadShaderMap must also be called with the same value, but from the game thread. */
	void SetRenderingThreadShaderMap(FComputeKernelShaderMap* InShaderMap);

	void AddCompileId(uint32 InIdentifier) 
	{
		OutstandingCompileShaderMapIds.Add(InIdentifier);
	}

	void SetInlineShaderMap(FComputeKernelShaderMap* InShaderMap)
	{
		checkSlow(IsInGameThread() || IsInAsyncLoadingThread() || IsInParallelLoadingThread());
		GameThreadShaderMap = InShaderMap;
		if (LIKELY(GameThreadShaderMap))
		{
			GameThreadShaderMap->GetResource()->SetOwnerName(GetOwnerFName());
		}
		bContainsInlineShaders = true;
		bLoadedCookedShaderMapId = true;
		CookedShaderMapId = InShaderMap->GetShaderMapId();
	}

	void RemoveOutstandingCompileId(const int32 InOldOutstandingCompileShaderMapId);

	/**
	* Get generated or written source code for the shader
	* @param OutSource - generated source code
	* @return - true on Success
	*/
	bool GetHLSLSource(FString& OutSource) const
	{
		OutSource = ShaderSource;
		return true;
	};

	const FString& GetHLSLSource() const
	{
		return ShaderSource;
	}

	const FString& GetEntryPoint() const
	{
		return ShaderEntryPoint;
	}

	const FString& GetFriendlyName() const { return FriendlyName; }

	int32 GetNumPermutations() const;

	void SetupResource(
		ERHIFeatureLevel::Type InFeatureLevel,
		FString const& InFriendlyName,
		FString const& InShaderEntryPoint,
		FString const& InShaderHashKey,
		FString const& InShaderSource,
		TMap<FString, FString> const& InAdditionalSources,
		TMap<FString, FString> const& InGeneratedSources,
		TSharedPtr<FComputeKernelDefinitionSet>& InShaderDefinitionSet,
		TSharedPtr<FComputeKernelPermutationVector>& InShaderPermutationVector,
		TUniquePtr<FShaderParametersMetadataAllocations>& InShaderParameterMetadataAllocations,
		FShaderParametersMetadata* InShaderParameterMetadata,
		FName const& InAssetPath
	);

#if WITH_EDITOR
	void SetupCompileEnvironment(int32 InPermutationId, FShaderCompilerEnvironment& OutShaderEnvironment) const;
#endif // WITH_EDITOR

	const FShaderParametersMetadata* GetShaderParamMetadata() const
	{
		return ShaderParameterMetadata;
	}
	
	TShaderRef<FComputeKernelShader> GetShader(int32 PermutationId) const;
	TShaderRef<FComputeKernelShader> GetShaderGameThread(int32 PermutationId) const;
	
	bool IsSame(const FComputeKernelShaderMapId& InId) const;

	uint32 GetKernelFlags() const { return KernelFlags; }

	FOnComputeKernelCompilationComplete& OnCompilationComplete() { return OnCompilationCompleteDelegate; }

protected:
#if WITH_EDITOR
	/**
	 * Fills the passed array with IDs of shader maps unfinished compilation jobs.
	 */
	void GetShaderMapIDsWithUnfinishedCompilation(TArray<int32>& OutShaderMapIds);
#endif // WITH_EDITOR

private:
	FComputeKernelCompileResults CompilationResults;

	/** 
	 * Game thread tracked shader map, which is ref counted and manages shader map lifetime. 
	 * The shader map uses deferred deletion so that the rendering thread has a chance to process a release command when the shader map is no longer referenced.
	 * Code that sets this is responsible for updating RenderingThreadShaderMap in a thread safe way.
	 * During an async compile, this will be NULL and will not contain the actual shader map until compilation is complete.
	 */
	TRefCountPtr<FComputeKernelShaderMap> GameThreadShaderMap = nullptr;

	/** 
	 * Shader map for this FComputeKernelResource which is accessible by the rendering thread. 
	 * This must be updated along with GameThreadShaderMap, but on the rendering thread.
	 */
	FComputeKernelShaderMap* RenderingThreadShaderMap = nullptr;

	/** Name of shader main function. */
	FString ShaderEntryPoint;

	/** Cached shader source for compilation. */
	FString ShaderSource;

	/** 
	 * Hash computed from shader code. If multiple kernels give
	 * the same output, same shader resource will be shared 
	 */
	uint64 ShaderCodeHash = 0;

	/** Additional source code. Stored as a map from a virtual source file name to the shader source.  */
	TMap<FString, FString> AdditionalSources;
	/** Generated source code. Stored as a map from a virtual source file name to the shader source.  */
	TMap<FString, FString> GeneratedSources;

	/** Defines used when compiling shaders. Object can be shared across a number of shader formats during cook. */
	TSharedPtr<FComputeKernelDefinitionSet> ShaderDefinitionSet;
	
	/** Permutations used for compiling shaders. Object can be shared across a number of shader formats during cook. */
	TSharedPtr<FComputeKernelPermutationVector> ShaderPermutationVector;

	/** Allocations for the ShaderParameterMetadata */
	TUniquePtr<FShaderParametersMetadataAllocations> ShaderParameterMetadataAllocations;

	/** Shader parameter metadata. It is expected that object is owned by ShaderParameterMetadataAllocations.  */
	FShaderParametersMetadata const* ShaderParameterMetadata;

	/** Compilation flags. */
	uint32 KernelFlags = 0;

	/** 
	 * Contains the compiling id of this shader map when it is being compiled asynchronously. 
	 * This can be used to access the shader map during async compiling, since GameThreadShaderMap will not have been set yet.
	 */
	TArray<int32, TInlineAllocator<1> > OutstandingCompileShaderMapIds;

	/** Feature level that this kernel is representing. */
	ERHIFeatureLevel::Type FeatureLevel = ERHIFeatureLevel::SM5;

	/**
	 * Whether this kernel was loaded with shaders inlined.
	 * If true, GameThreadShaderMap will contain a reference to the inlined shader map between Serialize and PostLoad.
	 */

	uint32 bContainsInlineShaders : 1;
	uint32 bLoadedCookedShaderMapId : 1;
	FComputeKernelShaderMapId CookedShaderMapId;

	FOnComputeKernelCompilationComplete OnCompilationCompleteDelegate;

#if WITH_EDITOR
	/**
	 * Compiles this kernel for InPlatform, storing the result in OutShaderMap if the compile was synchronous
	 */
	bool BeginCompileShaderMap(
		const FComputeKernelShaderMapId& InShaderMapId,
		EShaderPlatform InPlatform, 
		TRefCountPtr<FComputeKernelShaderMap>& OutShaderMap, 
		bool bApplyCompletedShaderMapForRendering,
		bool bSynchronous = false);

	/** Populates OutEnvironment with defines needed to compile shaders for this kernel. */
	void SetupShaderCompilationEnvironment(
		EShaderPlatform InPlatform,
		FShaderCompilerEnvironment& OutEnvironment
		) const;
#endif // WITH_EDITOR

	FString FriendlyName;

	/** Asset using this resource */
	FName AssetPath;

	friend class FComputeKernelShaderMap;
	friend class FShaderCompilingManager;
};


