// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ComputeSystemInterface.h"
#include "Containers/Map.h"

class FComputeGraphTaskWorker;
class FComputeViewExtension;

/** Implementation of IComputeSystem which manages FComputeGraphTaskWorker objects. */
class FComputeFrameworkSystem : public IComputeSystem
{
public:
	/** IComputeSystem implementation */
	void CreateWorkers(FSceneInterface const* InScene, TArray<IComputeTaskWorker*>& OutWorkers) override;
	void DestroyWorkers(FSceneInterface const* InScene, TArray<IComputeTaskWorker*>& InOutWorkers) override;

	/** Get the FComputeGraphTaskWorker object associated with the Scene. */
	FComputeGraphTaskWorker* GetComputeWorker(FSceneInterface const* InScene) const;

private:
	TMap<FSceneInterface const*, FComputeGraphTaskWorker*> ComputeWorkers;
	TMap<FSceneInterface const*, TSharedPtr<FComputeViewExtension, ESPMode::ThreadSafe>> ViewExtensions;
};
