// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SceneViewExtension.h"

/*
 * Compute view extension, useful for receiving callbacks from renderer.
 */
class FComputeViewExtension : public FWorldSceneViewExtension
{
public:
	FComputeViewExtension(const FAutoRegister& AutoReg, UWorld* InWorld);

	//~ Begin FSceneViewExtensionBase implementation
	virtual void PostTLASBuild_RenderThread(FRDGBuilder& GraphBuilder, FSceneView& InView) override;
	virtual ESceneViewExtensionFlags GetFlags() const override;
	//~ End FSceneViewExtensionBase implementation

private:
	// Local to methods but stored here and emptied at each usage to avoid per frame array allocations.
	mutable TArray<IComputeTaskWorker*> ComputeWorkers;
};
