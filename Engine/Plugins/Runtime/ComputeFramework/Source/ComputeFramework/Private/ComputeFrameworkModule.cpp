// Copyright Epic Games, Inc. All Rights Reserved.

#include "ComputeFramework/ComputeFrameworkModule.h"

#include "ComputeFramework/ComputeSystem.h"
#include "ComputeSystemInterface.h"
#include "Interfaces/IPluginManager.h"
#include "Misc/Paths.h"
#include "Modules/ModuleManager.h"
#include "ShaderCore.h"
#include "ComputeFramework/ComputeFrameworkObjectVersion.h"
#include "UObject/DevObjectVersion.h"

// Unique serialization id for ComputeFramework
const FGuid FComputeFrameworkObjectVersion::GUID(0x6304a3e7, 0x00594f59, 0x8cfc21bd, 0x7721fd4e);

static FDevVersionRegistration GRegisterComputeFrameworkObjectVersion(FComputeFrameworkObjectVersion::GUID, FComputeFrameworkObjectVersion::LatestVersion, TEXT("Dev-ComputeFramework"));

FComputeFrameworkSystem* FComputeFrameworkModule::ComputeSystem = nullptr;

void FComputeFrameworkModule::StartupModule()
{
	FString PluginShaderDir = FPaths::Combine(IPluginManager::Get().FindPlugin(TEXT("ComputeFramework"))->GetBaseDir(), TEXT("Shaders"));
	AddShaderSourceDirectoryMapping(TEXT("/Plugin/ComputeFramework"), PluginShaderDir);

	ensure(ComputeSystem == nullptr);
	ComputeSystem = new class FComputeFrameworkSystem;
	ComputeSystemInterface::RegisterSystem(ComputeSystem);
}

void FComputeFrameworkModule::ShutdownModule()
{
	ensure(ComputeSystem != nullptr);
	ComputeSystemInterface::UnregisterSystem(ComputeSystem);
	delete ComputeSystem;
	ComputeSystem = nullptr;
}

IMPLEMENT_MODULE(FComputeFrameworkModule, ComputeFramework)
