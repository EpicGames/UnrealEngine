// Copyright Epic Games, Inc. All Rights Reserved.

#include "ComputeFramework/ComputeGraph.h"

#include "Components/ActorComponent.h"
#include "ComputeFramework/ComputeDataInterface.h"
#include "ComputeFramework/ComputeDataProvider.h"
#include "ComputeFramework/ComputeFramework.h"
#include "ComputeFramework/ComputeGraphRenderProxy.h"
#include "ComputeFramework/ComputeKernelPermutationSet.h"
#include "ComputeFramework/ComputeKernelPermutationVector.h"
#include "ComputeFramework/ComputeKernel.h"
#include "ComputeFramework/ComputeKernelShared.h"
#include "ComputeFramework/ComputeKernelSource.h"
#include "ComputeFramework/ComputeSource.h"
#include "ComputeFramework/ShaderParamTypeDefinition.h"
#include "ComputeFramework/ShaderParameterMetadataAllocation.h"
#include "DataDrivenShaderPlatformInfo.h"
#include "GameFramework/Actor.h"
#include "Interfaces/ITargetPlatform.h"
#include "Misc/App.h"
#include "Serialization/CompactBinarySerialization.h"
#include "Serialization/CompactBinaryWriter.h"
#include "ShaderParameterMetadataBuilder.h"
#include "UObject/ObjectSaveContext.h"
#include "UObject/Package.h"

#if WITH_EDITORONLY_DATA
#include "Cooker/CookDependencyContext.h"
#include "Cooker/CookEvents.h"
#endif

#include UE_INLINE_GENERATED_CPP_BY_NAME(ComputeGraph)

UComputeGraph::UComputeGraph() = default;

UComputeGraph::UComputeGraph(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

UComputeGraph::UComputeGraph(FVTableHelper& Helper)
	: Super(Helper)
{
}

void UComputeGraph::BeginDestroy()
{
	Super::BeginDestroy();
	
#if WITH_EDITORONLY_DATA
	// Cancel any outstanding compilation jobs. These have a pointer to resource ShaderMetadata about to be deleted.
	for (FComputeKernelResourceSet& KernelResource : KernelResources)
	{
		KernelResource.CancelCompilation();
	}
#endif

	// Release on render thread. No need to wait on this before continuing destroy.
	ReleaseRenderProxy(RenderProxy);
}

void UComputeGraph::Serialize(FArchive& Ar)
{
	Super::Serialize(Ar);

	int32 NumKernels = 0;
	if (Ar.IsSaving())
	{
		NumKernels = KernelResources.Num();
	}
	Ar << NumKernels;
	if (Ar.IsLoading())
	{
		KernelResources.SetNum(NumKernels);
	}

	for (int32 KernelIndex = 0; KernelIndex < NumKernels; ++KernelIndex)
	{
		KernelResources[KernelIndex].Serialize(Ar);
	}
}

void UComputeGraph::PostLoad()
{
	Super::PostLoad();

	for (UComputeDataInterface* DataInterface : DataInterfaces)
	{
		if (DataInterface)
		{
			DataInterface->ConditionalPostLoad();
		}
	}
	
#if WITH_EDITORONLY_DATA
	// PostLoad our kernel dependencies before any compiling.
	for (UComputeKernel* Kernel : KernelInvocations)
	{
		if (Kernel != nullptr)
		{
			Kernel->ConditionalPostLoad();
		}
	}

	for (FComputeKernelResourceSet& KernelResource : KernelResources)
	{
		KernelResource.ProcessSerializedShaderMaps();
	}
#endif

	if (!ComputeFramework::IsDeferredCompilation())
	{
		// Sync compile here so that any downstream tasks like automation tests don't need to wait for shader compilation
		constexpr bool bSync = true;
		UpdateResources(bSync);
	}
}

#if WITH_EDITORONLY_DATA

/** Shader dependency serialization code used for incremental cooking. */
namespace ComputeFramework::Private
{ 
	constexpr int32 HashDependenciesForCookArgsVersion = 1;
	
	void HashDependenciesForCook(FCbFieldViewIterator Args, UE::Cook::FCookDependencyContext& Context)
	{
		FCbFieldViewIterator ArgField(Args);
		int32 ArgsVersion = -1;
		ArgsVersion = (ArgField++).AsInt32();

		bool bValid = ArgsVersion == HashDependenciesForCookArgsVersion;
		
		TArray<FName> ShaderFormats;
		if (bValid)
		{
			bValid = LoadFromCompactBinary(ArgField++, ShaderFormats);
		}
		
		TArray<FString> ShaderVirtualPaths;
		if (bValid)
		{
			bValid = LoadFromCompactBinary(ArgField++, ShaderVirtualPaths);
		}

		if (!bValid)
		{
			Context.LogInvalidated(FString::Printf(TEXT("Failed to serialize UComputeGraph cook dependencies. ArgsVersion %d."), ArgsVersion));
			return;
		}

		for (FName ShaderFormat : ShaderFormats)
		{
			const EShaderPlatform ShaderPlatform = ShaderFormatToLegacyShaderPlatform(ShaderFormat);

			for (FString const& ShaderVirtualPath : ShaderVirtualPaths)
			{
				const FSHAHash Hash = GetShaderFileHash(*ShaderVirtualPath, ShaderPlatform);
				Context.Update(&Hash.Hash, sizeof(Hash.Hash));
			}
		}
	}
}

UE_COOK_DEPENDENCY_FUNCTION(HashComputeGraphDependenciesForCook, ComputeFramework::Private::HashDependenciesForCook);

#endif // WITH_EDITORONLY_DATA


#if WITH_EDITORONLY_DATA
void UComputeGraph::OnCookEvent(UE::Cook::ECookEvent CookEvent, UE::Cook::FCookEventContext& CookContext)
{
	Super::OnCookEvent(CookEvent, CookContext);
	if (CookEvent == UE::Cook::ECookEvent::PlatformCookDependencies && CookContext.IsCooking())
	{
		TArray<FName> ShaderFormats;
		if (ITargetPlatform const* TargetPlatform = CookContext.GetTargetPlatform())
		{
			TargetPlatform->GetAllTargetedShaderFormats(ShaderFormats);
		}
		
		TArray<FString> ShaderVirtualPaths;
		ShaderVirtualPaths.Reserve(DataInterfaces.Num() + 1);
		
		// Add dependency on compute graph entry shader file (and its includes).
		ShaderVirtualPaths.Add(TEXT("/Plugin/ComputeFramework/Private/ComputeKernel.usf"));
		
		// Add dependency on data interface shader template files (and their includes).
		for (UComputeDataInterface const* DataInterface : DataInterfaces)
		{
			if (DataInterface != nullptr)
			{
				if (TCHAR const* ShaderVirtualPath = DataInterface->GetShaderVirtualPath())
				{
					ShaderVirtualPaths.Add(ShaderVirtualPath);
				}
			}
		}

		FCbWriter Writer;
		Writer << ComputeFramework::Private::HashDependenciesForCookArgsVersion;
		Writer << ShaderFormats;
		Writer << ShaderVirtualPaths;

		CookContext.AddLoadBuildDependency(UE::Cook::FCookDependency::Function(UE_COOK_DEPENDENCY_FUNCTION_CALL(HashComputeGraphDependenciesForCook), Writer.Save()));
	}
}
#endif

bool UComputeGraph::ValidateGraph(FString* OutErrors)
{
	bool bValid = true;

	// Validate that we have exactly one execution provider per kernel.
	TArray<bool, TInlineAllocator<64>> KernelHasExecution;
	KernelHasExecution.SetNumZeroed(KernelInvocations.Num());
	for (FComputeGraphEdge const& Edge : GraphEdges)
	{
		if (!KernelInvocations.IsValidIndex(Edge.KernelIndex))
		{
			continue;
		}
		if (DataInterfaces.IsValidIndex(Edge.DataInterfaceIndex) && DataInterfaces[Edge.DataInterfaceIndex] != nullptr
			&& DataInterfaces[Edge.DataInterfaceIndex]->IsExecutionInterface())
		{
			if (KernelHasExecution[Edge.KernelIndex])
			{
				if (OutErrors && KernelInvocations.IsValidIndex(Edge.KernelIndex)
					&& KernelInvocations[Edge.KernelIndex] && KernelInvocations[Edge.KernelIndex]->KernelSource)
				{
					OutErrors->Appendf(TEXT("Kernel '%s': multiple execution interfaces connected.\n"),
						*KernelInvocations[Edge.KernelIndex]->KernelSource->EntryPoint);
				}
				return false;
			}
			KernelHasExecution[Edge.KernelIndex] = true;
		}
	}
	for (int32 KernelIndex = 0; KernelIndex < KernelInvocations.Num(); ++KernelIndex)
	{
		if (KernelInvocations[KernelIndex] != nullptr && !KernelHasExecution[KernelIndex])
		{
			if (OutErrors && KernelInvocations[KernelIndex]->KernelSource)
			{
				OutErrors->Appendf(TEXT("Kernel '%s': no execution interface (e.g. Dispatch) connected. Bind a pin to a Dispatch interface.\n"),
					*KernelInvocations[KernelIndex]->KernelSource->EntryPoint);
			}
			bValid = false;
		}
	}

	// Check that no two edges occupy the same kernel binding slot (duplicate bindings).
	{
		TSet<uint64> InputsSeen, OutputsSeen;
		for (FComputeGraphEdge const& Edge : GraphEdges)
		{
			const uint64 Key = ((uint64)(uint32)Edge.KernelIndex << 32) | (uint32)Edge.KernelBindingIndex;
			TSet<uint64>& Seen = Edge.bKernelInput ? InputsSeen : OutputsSeen;
			bool bAlreadyInSet = false;
			Seen.FindOrAdd(Key, &bAlreadyInSet);
			if (bAlreadyInSet)
			{
				if (OutErrors && KernelInvocations.IsValidIndex(Edge.KernelIndex)
					&& KernelInvocations[Edge.KernelIndex] && KernelInvocations[Edge.KernelIndex]->KernelSource)
				{
					OutErrors->Appendf(TEXT("Kernel '%s': duplicate %s binding at index %d.\n"),
						*KernelInvocations[Edge.KernelIndex]->KernelSource->EntryPoint,
						Edge.bKernelInput ? TEXT("input") : TEXT("output"),
						Edge.KernelBindingIndex);
				}
				bValid = false;
			}
		}
	}

	// Check that each edge's DataInterfaceBindingIndex is valid and (editor-only) that function types match.
	for (FComputeGraphEdge const& Edge : GraphEdges)
	{
		if (!DataInterfaces.IsValidIndex(Edge.DataInterfaceIndex) || !DataInterfaces[Edge.DataInterfaceIndex])
		{
			continue;
		}

		TArray<FShaderFunctionDefinition> Functions;
		if (Edge.bKernelInput)
		{
			DataInterfaces[Edge.DataInterfaceIndex]->GetSupportedInputs(Functions);
		}
		else
		{
			DataInterfaces[Edge.DataInterfaceIndex]->GetSupportedOutputs(Functions);
		}

		if (!Functions.IsValidIndex(Edge.DataInterfaceBindingIndex))
		{
			if (OutErrors)
			{
				OutErrors->Appendf(TEXT("Edge references out-of-range data interface binding index %d (max %d).\n"),
					Edge.DataInterfaceBindingIndex, Functions.Num() - 1);
			}
			bValid = false;
			continue;
		}

#if WITH_EDITORONLY_DATA
		if (KernelInvocations.IsValidIndex(Edge.KernelIndex) && KernelInvocations[Edge.KernelIndex]	&& KernelInvocations[Edge.KernelIndex]->KernelSource)
		{
			TArray<FShaderFunctionDefinition> const& KernelFns = Edge.bKernelInput
				? KernelInvocations[Edge.KernelIndex]->KernelSource->ExternalInputs
				: KernelInvocations[Edge.KernelIndex]->KernelSource->ExternalOutputs;

			if (KernelFns.IsValidIndex(Edge.KernelBindingIndex))
			{
				FShaderFunctionDefinition const& DIFn = Functions[Edge.DataInterfaceBindingIndex];
				FShaderFunctionDefinition const& KernelFn = KernelFns[Edge.KernelBindingIndex];

				bool bParamTypesMatch = (DIFn.ParamTypes.Num() == KernelFn.ParamTypes.Num());
				for (int32 ParamIdx = 0; bParamTypesMatch && ParamIdx < DIFn.ParamTypes.Num(); ++ParamIdx)
				{
					bParamTypesMatch = DIFn.ParamTypes[ParamIdx].TypeDeclaration == KernelFn.ParamTypes[ParamIdx].TypeDeclaration;
				}

				if (!bParamTypesMatch)
				{
					if (OutErrors)
					{
						OutErrors->Appendf(TEXT("Kernel '%s': %s pin %d type mismatch with data interface function '%s'.\n"),
							*KernelInvocations[Edge.KernelIndex]->KernelSource->EntryPoint,
							Edge.bKernelInput ? TEXT("input") : TEXT("output"),
							Edge.KernelBindingIndex,
							*DIFn.Name);
					}
					bValid = false;
				}
			}
		}
#endif
	}

	// Check that the kernel dependency graph is a DAG (no cycles).
	{
		const int32 NumKernels = KernelInvocations.Num();
		TArray<TArray<int32>> Adj;
		Adj.SetNum(NumKernels);

		TMap<int32, TArray<int32>> DiWriters; // DataInterfaceIndex to kernels that output to it.
		for (FComputeGraphEdge const& Edge : GraphEdges)
		{
			if (!Edge.bKernelInput && KernelInvocations.IsValidIndex(Edge.KernelIndex))
			{
				DiWriters.FindOrAdd(Edge.DataInterfaceIndex).AddUnique(Edge.KernelIndex);
			}
		}
		for (FComputeGraphEdge const& Edge : GraphEdges)
		{
			if (Edge.bKernelInput && KernelInvocations.IsValidIndex(Edge.KernelIndex))
			{
				if (TArray<int32>* Writers = DiWriters.Find(Edge.DataInterfaceIndex))
				{
					for (int32 WriterKernel : *Writers)
					{
						if (WriterKernel != Edge.KernelIndex)
						{
							Adj[WriterKernel].AddUnique(Edge.KernelIndex);
						}
					}
				}
			}
		}

		// Three-color DFS: 0=unvisited, 1=in-stack, 2=done.
		TArray<uint8> Color;
		Color.SetNumZeroed(NumKernels);

		TFunction<void(int32)> Dfs = [&](int32 Node)
		{
			Color[Node] = 1;
			for (int32 Next : Adj[Node])
			{
				if (Color[Next] == 1)
				{
					if (OutErrors && KernelInvocations.IsValidIndex(Node) && KernelInvocations[Node]
						&& KernelInvocations[Node]->KernelSource
						&& KernelInvocations.IsValidIndex(Next) && KernelInvocations[Next]
						&& KernelInvocations[Next]->KernelSource)
					{
						OutErrors->Appendf(TEXT("Graph cycle detected: kernel '%s' to '%s'.\n"),
							*KernelInvocations[Node]->KernelSource->EntryPoint,
							*KernelInvocations[Next]->KernelSource->EntryPoint);
					}
					bValid = false;
				}
				else if (Color[Next] == 0)
				{
					Dfs(Next);
				}
			}
			Color[Node] = 2;
		};

		for (int32 KernelIndex = 0; KernelIndex < NumKernels; ++KernelIndex)
		{
			if (Color[KernelIndex] == 0)
			{
				Dfs(KernelIndex);
			}
		}
	}

	return bValid;
}

bool UComputeGraph::ValidateProviders(TArray< TObjectPtr<UComputeDataProvider> > const& DataProviders) const
{
	if (DataInterfaces.Num() != DataProviders.Num())
	{
		return false;
	}
	for (int32 Index = 0; Index < DataInterfaces.Num(); ++Index)
	{
		if (DataProviders[Index] == nullptr && DataInterfaces[Index] != nullptr)
		{
			return false;
		}
	}
	return true;
}

namespace ComputeFramework::Private
{
	TMap<int32, UComputeDataProvider::FInputOutputMask> BuildPerKernelInputOutputMasks(
		int32 InDataInterfaceIndex, TArray<FComputeGraphEdge> const& InGraphEdges)
	{
		TMap<int32, UComputeDataProvider::FInputOutputMask> Result;
		for (FComputeGraphEdge const& GraphEdge : InGraphEdges)
		{
			if (GraphEdge.DataInterfaceIndex != InDataInterfaceIndex)
			{
				continue;
			}

			UComputeDataProvider::FInputOutputMask& Entry = Result.FindOrAdd(GraphEdge.KernelIndex);
			if (GraphEdge.bKernelInput)
			{
				Entry.InputMask |= 1llu << GraphEdge.DataInterfaceBindingIndex;
			}
			else
			{
				Entry.OutputMask |= 1llu << GraphEdge.DataInterfaceBindingIndex;
			}
		}
		return Result;
	}
}

void UComputeGraph::CreateDataProviders(int32 InBindingIndex, TObjectPtr<UObject>& InBindingObject, TArray< TObjectPtr<UComputeDataProvider> >& InOutDataProviders) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UComputeGraph::CreateDataProviders);

	InOutDataProviders.SetNumZeroed(DataInterfaces.Num());

	if (ensure(Bindings.IsValidIndex(InBindingIndex)) && (!InBindingObject || InBindingObject.IsA(Bindings[InBindingIndex])))
	{
		for (int32 DataInterfaceIndex = 0; DataInterfaceIndex < DataInterfaces.Num(); ++DataInterfaceIndex)
		{
			if (ensure(DataInterfaceToBinding.IsValidIndex(DataInterfaceIndex)) && DataInterfaceToBinding[DataInterfaceIndex] == InBindingIndex)
			{
				UComputeDataProvider* DataProvider = nullptr;

				UComputeDataInterface const* DataInterface = DataInterfaces[DataInterfaceIndex];
				if (DataInterface != nullptr)
				{
					// Gather which input/output bindings are connected in the graph.
					uint64 InputMask = 0;
					uint64 OutputMask = 0;
					GetDataInterfaceInputOutputMasks(DataInterfaceIndex, InputMask, OutputMask);

					DataProvider = DataInterface->CreateDataProvider();

					if (DataProvider)
					{
						DataProvider->Initialize(DataInterfaceIndex, DataInterface, InBindingObject, InputMask, OutputMask);
					}
					PRAGMA_DISABLE_DEPRECATION_WARNINGS
					else
					{
						// Legacy fall back - try to use previous CreateDataProvider path.
						DataProvider = DataInterface->CreateDataProvider(InBindingObject, InputMask, OutputMask);
					}
					PRAGMA_ENABLE_DEPRECATION_WARNINGS

					if (DataProvider && DataInterface->GetRequiresPerKernelInputOutputMasks())
					{
						DataProvider->InitializePerKernelInputOutputMasks(
							ComputeFramework::Private::BuildPerKernelInputOutputMasks(DataInterfaceIndex, GraphEdges));
					}
				}

				InOutDataProviders[DataInterfaceIndex] = DataProvider;
			}
		}
	}
}

void UComputeGraph::InitializeDataProviders(int32 InBindingIndex, TObjectPtr<UObject>& InBindingObject, TArray< TObjectPtr<UComputeDataProvider> >& InDataProviders) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UComputeGraph::InitializeDataProviders);

	if (ensure(Bindings.IsValidIndex(InBindingIndex)) && (!InBindingObject || InBindingObject.IsA(Bindings[InBindingIndex])))
	{
		for (int32 DataInterfaceIndex = 0; DataInterfaceIndex < DataInterfaces.Num(); ++DataInterfaceIndex)
		{
			if (ensure(DataInterfaceToBinding.IsValidIndex(DataInterfaceIndex)) && DataInterfaceToBinding[DataInterfaceIndex] == InBindingIndex
				&& ensure(InDataProviders.IsValidIndex(DataInterfaceIndex)) && ensure(InDataProviders[DataInterfaceIndex]))
			{
				UComputeDataInterface const* DataInterface = DataInterfaces[DataInterfaceIndex];
				if (DataInterface != nullptr)
				{
					// Gather which input/output bindings are connected in the graph.
					uint64 InputMask = 0;
					uint64 OutputMask = 0;
					GetDataInterfaceInputOutputMasks(DataInterfaceIndex, InputMask, OutputMask);

					InDataProviders[DataInterfaceIndex]->Initialize(DataInterface, InBindingObject, InputMask, InputMask);

					if (DataInterface->GetRequiresPerKernelInputOutputMasks())
					{
						InDataProviders[DataInterfaceIndex]->InitializePerKernelInputOutputMasks(
							ComputeFramework::Private::BuildPerKernelInputOutputMasks(DataInterfaceIndex, GraphEdges));
					}
				}
			}
		}
	}
}

/** Compute Kernel compilation flags. */
enum class EComputeKernelCompilationFlags
{
	None = 0,

	/* Force recompilation even if kernel is not dirty and/or DDC data is available. */
	Force = 1 << 0,

	/* Compile the shader while blocking the main thread. */
	Synchronous = 1 << 1,

	/* Replaces all instances of the shader with the newly compiled version. */
	ApplyCompletedShaderMapForRendering = 1 << 2,
};

void UComputeGraph::UpdateResources(bool bSync)
{
#if WITH_EDITORONLY_DATA
	uint32 CompilationFlags = (uint32)EComputeKernelCompilationFlags::ApplyCompletedShaderMapForRendering;

	CompilationFlags |= bSync ? (uint32)EComputeKernelCompilationFlags::Synchronous : (uint32)EComputeKernelCompilationFlags::None;

	CacheResourceShadersForRendering(CompilationFlags);
#endif

	ReleaseRenderProxy(RenderProxy);
	RenderProxy = CreateRenderProxy();
}

bool UComputeGraph::HasKernelResourcesPendingShaderCompilation() const
{
	return !KernelResourceIndicesPendingShaderCompilation.IsEmpty();
}

FComputeGraphRenderProxy const* UComputeGraph::GetRenderProxy() const 
{
	return RenderProxy;
}

namespace
{
	/** 
	 * Get the unique name that will be used for shader bindings. 
	 * Multiple instances of the same data interface may be in a single graph, so we need to use an additional index to disambiguoate.
	 */
	FString GetUniqueDataInterfaceName(UComputeDataInterface const* InDataInterface, int32 InUniqueIndex)
	{
		check(InDataInterface != nullptr && InDataInterface->GetClassName() != nullptr);
		return FString::Printf(TEXT("DI%d_%s"), InUniqueIndex, InDataInterface->GetClassName());
	}
}

FShaderParametersMetadata* UComputeGraph::BuildKernelShaderMetadata(int32 InKernelIndex, FShaderParametersMetadataAllocations& InOutAllocations) const
{
	// Gather relevant data interfaces.
	TArray<int32> DataInterfaceIndices;
	for (FComputeGraphEdge const& GraphEdge : GraphEdges)
	{
		if (GraphEdge.KernelIndex == InKernelIndex)
		{
			DataInterfaceIndices.AddUnique(GraphEdge.DataInterfaceIndex);
		}
	}

	// Extract shader parameter info from data interfaces.
	FShaderParametersMetadataBuilder Builder;

	for (int32 DataInterfaceIndex : DataInterfaceIndices)
	{
		UComputeDataInterface const* DataInterface = DataInterfaces[DataInterfaceIndex];
		if (DataInterface != nullptr)
		{
			// Unique name needs to persist since it is directly referenced by shader metadata.
			// Allocate and store the string in InOutAllocations which should have the same lifetime as our return FShaderParametersMetadata object.
			int32 Index = InOutAllocations.Names.Add();
			InOutAllocations.Names[Index] = GetUniqueDataInterfaceName(DataInterface, DataInterfaceIndex);
			TCHAR const* NamePtr = *InOutAllocations.Names[Index];

			DataInterface->GetShaderParameters(NamePtr, Builder, InOutAllocations);
		}
	}

	// Graph name needs to persist since it's referenced by the metadata
	int32 GraphNameIndex = InOutAllocations.Names.Add();
	InOutAllocations.Names[GraphNameIndex] = GetName();
	FShaderParametersMetadata* ShaderParameterMetadata = Builder.Build(FShaderParametersMetadata::EUseCase::ShaderParameterStruct, *InOutAllocations.Names[GraphNameIndex]);
	InOutAllocations.ShaderParameterMetadatas.Add(ShaderParameterMetadata);

	return ShaderParameterMetadata;
}

void UComputeGraph::BuildShaderPermutationVectors(TArray<FComputeKernelPermutationVector>& OutShaderPermutationVectors) const
{
	if (FApp::CanEverRender())
	{
		OutShaderPermutationVectors.Reset();
		OutShaderPermutationVectors.AddDefaulted(KernelInvocations.Num());

		TSet<uint64> Found;
		for (FComputeGraphEdge const& GraphEdge : GraphEdges)
		{
			if (DataInterfaces[GraphEdge.DataInterfaceIndex] != nullptr)
			{
				uint64 PackedFoundValue = ((uint64)GraphEdge.DataInterfaceIndex << 32) | (uint64)GraphEdge.KernelIndex;
				if (!Found.Find(PackedFoundValue))
				{
					DataInterfaces[GraphEdge.DataInterfaceIndex]->GetPermutations(OutShaderPermutationVectors[GraphEdge.KernelIndex]);
					Found.Add(PackedFoundValue);
				}
			}
		}
	}
}

FComputeGraphRenderProxy* UComputeGraph::CreateRenderProxy() const
{
	// Rendering is disabled, so no need to create the render proxy
	if (!FApp::CanEverRender())
	{
		return nullptr;
	}
	
	FComputeGraphRenderProxy* Proxy = new FComputeGraphRenderProxy;
	Proxy->GraphName = GetFName();
	Proxy->ShaderParameterMetadataAllocations = MakeUnique<FShaderParametersMetadataAllocations>();

	BuildShaderPermutationVectors(Proxy->ShaderPermutationVectors);

	const int32 NumKernels = KernelInvocations.Num();
	Proxy->KernelInvocations.Reserve(NumKernels);

	for (int32 KernelIndex = 0; KernelIndex < NumKernels; ++KernelIndex)
	{
		UComputeKernel const* Kernel = KernelInvocations[KernelIndex];
		FComputeKernelResource const* KernelResource = KernelResources[KernelIndex].Get();

		if (Kernel != nullptr && KernelResource != nullptr)
		{
			FComputeGraphRenderProxy::FKernelInvocation& Invocation = Proxy->KernelInvocations.AddDefaulted_GetRef();

			Invocation.KernelName = Kernel->KernelSource->EntryPoint;
			Invocation.KernelGroupSize = Kernel->KernelSource->GroupSize;
			Invocation.KernelResource = KernelResource;
			Invocation.ShaderParameterMetadata = BuildKernelShaderMetadata(KernelIndex, *Proxy->ShaderParameterMetadataAllocations);
			Invocation.bSupportsUnifiedDispatch = true;
			Invocation.bTriggerRenderCapture = !!(Kernel->KernelFlags & (int32)EComputeKernelFlags::TriggerRenderCapture);

			// 1. Figure out which data interface is the execution data interface
			for (FComputeGraphEdge const& GraphEdge : GraphEdges)
			{
				if (GraphEdge.KernelIndex == KernelIndex)
				{
					UComputeDataInterface const* DataInterface = DataInterfaces[GraphEdge.DataInterfaceIndex];
					if (ensure(DataInterface != nullptr))
					{
						if (DataInterface->IsExecutionInterface())
						{
							Invocation.ExecutionProviderIndex = GraphEdge.DataInterfaceIndex;
							break;
						}
					}
				}
			}

			// 1. Data interfaces sharing the same binding (primary) as the kernel should present its data in a way that
			// matches the kernel dispatch method, which can be either unified(full buffer) or non-unified (per invocation window into the full buffer)
			// 2. Data interfaces not sharing the same binding (secondary) should always provide a full view to its data (unified)
			// Note: In case of non-unified kernel, extra work maybe needed to read from secondary buffers.
			// When kernel is non-unified, index = 0...section.max for each invocation/section, 
			// so user may want to consider using a dummy buffer that maps section index to the indices of secondary buffers
			// for example, given a non-unified kernel, primary and secondary components sharing the same vertex count, we might want to create a buffer
			// in the primary group that is simply [0,1,2...,NumVerts-1], which we can then index into to map section vert index to the global vert index
			if (ensure(DataInterfaceToBinding.IsValidIndex(Invocation.ExecutionProviderIndex)))
			{
				int32 ExecutionComponentBindingIndex = DataInterfaceToBinding[Invocation.ExecutionProviderIndex];

				for (FComputeGraphEdge const& GraphEdge : GraphEdges)
				{
					if (GraphEdge.KernelIndex == KernelIndex)
					{
						UComputeDataInterface const* DataInterface = DataInterfaces[GraphEdge.DataInterfaceIndex];
						if (ensure(DataInterface != nullptr))
						{
							const int32 DataInterfaceComponentBindingIndex = DataInterfaceToBinding[GraphEdge.DataInterfaceIndex];
							const bool bIsPrimary = DataInterfaceComponentBindingIndex == ExecutionComponentBindingIndex;
							
							const int32 IndexOfIndex = Invocation.BoundProviderIndices.AddUnique(GraphEdge.DataInterfaceIndex);

							// Added a new provider, store whether it is primary or secondary
							if (IndexOfIndex == Invocation.BoundProviderIsPrimary.Num())
							{
								Invocation.BoundProviderIsPrimary.Add( bIsPrimary );
							}	

							// Only Data Interfaces in the primary group should determine the kernel dispatch type
							if (bIsPrimary)
							{
								Invocation.bSupportsUnifiedDispatch &= DataInterface->CanSupportUnifiedDispatch();
							}

							// If the data interface is requesting readback and is a kernel output, register it.
							if (!GraphEdge.bKernelInput)
							{
								if (DataInterface->GetRequiresReadback())
								{
									Invocation.ReadbackProviderIndices.AddUnique(GraphEdge.DataInterfaceIndex);
								}

								if (DataInterface->GetRequiresPreSubmitCall())
								{
									Invocation.PreSubmitProviderIndices.AddUnique(GraphEdge.DataInterfaceIndex);
								}

								if (DataInterface->GetRequiresPostSubmitCall())
								{
									Invocation.PostSubmitProviderIndices.AddUnique(GraphEdge.DataInterfaceIndex);
								}
							}
						}
					}
				}

				check(Invocation.BoundProviderIndices.Num() == Invocation.BoundProviderIsPrimary.Num());
			}
		}
	}

	return Proxy;
}

void UComputeGraph::ReleaseRenderProxy(FComputeGraphRenderProxy* InRenderProxy) const
{
	if (InRenderProxy != nullptr)
	{
		// Serialize release on render thread in case proxy is being accessed there.
		ENQUEUE_RENDER_COMMAND(ReleaseRenderProxy)([InRenderProxy](FRHICommandListImmediate& RHICmdList)
		{
			delete InRenderProxy;
		});
	}
}

void UComputeGraph::GetDataInterfaceInputOutputMasks(int32 InDataInterfaceIndex, uint64& OutInputMask, uint64& OutOutputMask) const
{
	for (FComputeGraphEdge const& GraphEdge : GraphEdges)
	{
		if (GraphEdge.DataInterfaceIndex == InDataInterfaceIndex)
		{
			if (GraphEdge.bKernelInput)
			{
				OutInputMask |= 1llu << GraphEdge.DataInterfaceBindingIndex;
			}
			else
			{
				OutOutputMask |= 1llu << GraphEdge.DataInterfaceBindingIndex;
			}
		}
	}
}

#if WITH_EDITORONLY_DATA

namespace
{
	/** Add HLSL code to implement an external function. */
	void GetFunctionShimHLSL(FShaderFunctionDefinition const& FnImpl, FShaderFunctionDefinition const& FnWrap, TCHAR const* UID, TCHAR const *WrapNameOverride, TCHAR const* Namespace, FString& InOutHLSL)
	{
		const bool bHasReturnImpl = FnImpl.bHasReturnType;
		const int32 NumImplParams = FnImpl.ParamTypes.Num();
		const int32 NumImplInputParams = bHasReturnImpl ? NumImplParams - 1 : NumImplParams;
		
		const bool bHasReturnWrap = FnWrap.bHasReturnType;
		const int32 NumWrapParams = FnWrap.ParamTypes.Num();

		TStringBuilder<512> StringBuilder;

		if (Namespace)
		{
			StringBuilder.Append(TEXT("namespace "));
			StringBuilder.Append(Namespace);
			StringBuilder.Append(TEXT(" { "));
		}
		StringBuilder.Append(bHasReturnWrap ? *FnWrap.ParamTypes[0].TypeDeclaration : TEXT("void"));
		StringBuilder.Append(TEXT(" "));
		StringBuilder.Append(WrapNameOverride ? WrapNameOverride : *FnWrap.Name);
		StringBuilder.Append(TEXT("("));
		
		for (int32 ParameterIndex = bHasReturnWrap ? 1 : 0; ParameterIndex < NumWrapParams; ++ParameterIndex)
		{
			switch (FnWrap.ParamTypes[ParameterIndex].Modifier)
			{
			case EShaderParamModifier::In:
				StringBuilder.Append(TEXT("in "));
				break;
			case EShaderParamModifier::Out:
				StringBuilder.Append(TEXT("out "));
				break;
			case EShaderParamModifier::InOut:
				StringBuilder.Append(TEXT("inout "));
				break;
			default:
				break;
			}

			StringBuilder.Append(*FnWrap.ParamTypes[ParameterIndex].TypeDeclaration);
			StringBuilder.Appendf(TEXT(" P%d"), ParameterIndex);
			StringBuilder.Append((ParameterIndex < NumWrapParams - 1) ? TEXT(", ") : TEXT(""));
		}

		StringBuilder.Append(TEXT(") { "));
		StringBuilder.Append(bHasReturnWrap ? TEXT("return ") : TEXT(""));
		StringBuilder.Append(*FnImpl.Name).Append(TEXT("_")).Append(UID);
		StringBuilder.Append(TEXT("("));

		// There are cases where the impl will have fewer input params than the wrap, additional wrap params should be skipped
		// Example: when a parameter pin connects to a resource pin
		// void Wrap(uint P0, uint P1, ...) { Impl(); } // Impl has no input param
		// SomeType Wrap(uint P1, uint P2, ...) { return Impl(P1); } // Impl has 1 input param
		int32 NumImplInputParamsUsed = 0;
		for (int32 WrapParameterIndex = bHasReturnWrap? 1 : 0; WrapParameterIndex < NumWrapParams; ++WrapParameterIndex)
		{
			if (NumImplInputParamsUsed >= NumImplInputParams)
			{
				break;
			}
			// prepend a comma if we are not the first param for the impl
			StringBuilder.Append((NumImplInputParamsUsed != 0) ? TEXT(", ") : TEXT(""));
			StringBuilder.Appendf(TEXT("P%d"), WrapParameterIndex);
			
			NumImplInputParamsUsed++;
		}

		StringBuilder.Append(TEXT(");"));
		
		if (Namespace)
		{
			StringBuilder.Append(TEXT(" }"));
		}
		StringBuilder.Append(TEXT(" }\n"));

		InOutHLSL += StringBuilder.ToString();
	}

	/** Add source includes to unique list, recursively adding additional sources. */
	void AddSourcesRecursive(TArray<UComputeSource*> const& InSources, TArray<UComputeSource const*>& InOutUniqueSources)
	{
		for (UComputeSource const* Source : InSources)
		{
			if (Source != nullptr)
			{
				if (!InOutUniqueSources.Contains(Source))
				{
					AddSourcesRecursive(Source->AdditionalSources, InOutUniqueSources);

					InOutUniqueSources.AddUnique(Source);
				}
			}
		}
	}

	/** Get source includes as map of include file name to HLSL source. */
	TMap<FString, FString> GatherAdditionalSources(TArray<UComputeSource*> const& InSources)
	{
		TMap<FString, FString> Result;

		TArray<UComputeSource const*> UniqueSources;
		AddSourcesRecursive(InSources, UniqueSources);

		for (UComputeSource const* Source : UniqueSources)
		{
			Result.Add(Source->GetVirtualPath(), Source->GetSource());
		}

		return Result;
	}
}

FString UComputeGraph::BuildKernelSource(
	int32 KernelIndex, 
	UComputeKernelSource const& InKernelSource,
	TMap<FString, FString> const& InAdditionalSources,
	FString& OutHashKey,
	TMap<FString, FString>& OutGeneratedSources,
	FComputeKernelDefinitionSet& OutDefinitionSet,
	FComputeKernelPermutationVector& OutPermutationVector) const
{
	FString HLSL;
	FSHA1 HashState;

	TSet<FString> StructsSeen;
	TArray<FString> StructDeclarations;

	// Add virtual source includes from the additional sources.
	for (TPair<FString, FString> const& AdditionalSource : InAdditionalSources)
	{
		HLSL += FString::Printf(TEXT("\n#include \"%s\"\n"), *AdditionalSource.Key);

		// Accumulate the source HLSL to the local hash state.
		HashState.UpdateWithString(*AdditionalSource.Value, AdditionalSource.Value.Len());
	}

	// Add defines and permutations.
	OutDefinitionSet = InKernelSource.DefinitionsSet;
	OutPermutationVector.AddPermutationSet(InKernelSource.PermutationSet);

	// Find associated data interfaces.
	TArray<int32> RelevantEdgeIndices;
	TArray<int32> DataProviderIndices;
	for (int32 GraphEdgeIndex = 0; GraphEdgeIndex < GraphEdges.Num(); ++GraphEdgeIndex)
	{
		if (GraphEdges[GraphEdgeIndex].KernelIndex == KernelIndex)
		{
			RelevantEdgeIndices.Add(GraphEdgeIndex);
			DataProviderIndices.AddUnique(GraphEdges[GraphEdgeIndex].DataInterfaceIndex);
		}
	}

	// Collect data interface shader code.
	for (int32 DataProviderIndex : DataProviderIndices)
	{
		UComputeDataInterface* DataInterface = DataInterfaces[DataProviderIndex];
		if (DataInterface != nullptr)
		{
			// Add a unique prefix to generate unique names in the data interface shader code.
			FString NamePrefix = GetUniqueDataInterfaceName(DataInterface, DataProviderIndex);

			// Data interface optionally put source in a generated file that maps to an on-disk virtual path.
			if (TCHAR const* ShaderVirtualPath = DataInterface->GetShaderVirtualPath())
			{
				// The generated path has a magic unique prefix which the compilation manager knows to strip before resolving errors.
				FString MagicVirtualPath = FString::Printf(TEXT("/Engine/Generated/DataInterface/%s%s"), *NamePrefix, ShaderVirtualPath);
				HLSL += FString::Printf(TEXT("\n#include \"%s\"\n"), *MagicVirtualPath);
				FString DataInterfaceHLSL;
				DataInterface->GetHLSL(DataInterfaceHLSL, NamePrefix);
				OutGeneratedSources.Add(MagicVirtualPath, DataInterfaceHLSL);
			}
			else
			{
				DataInterface->GetHLSL(HLSL, NamePrefix);
			}

			DataInterface->GetStructDeclarations(StructsSeen, StructDeclarations);
			
			// Get define and permutation info for each data provider.
			DataInterface->GetDefines(OutDefinitionSet);
			DataInterface->GetPermutations(OutPermutationVector);

			// Add contribution from the data provider to the final hash key.
			DataInterface->GetShaderHash(OutHashKey);
		}
	}

	// Bind every external kernel function to the associated data input function.
	for (int32 GraphEdgeIndex : RelevantEdgeIndices)
	{
		FComputeGraphEdge const& GraphEdge = GraphEdges[GraphEdgeIndex];
		if (DataInterfaces[GraphEdge.DataInterfaceIndex] != nullptr)
		{
			FString NamePrefix = GetUniqueDataInterfaceName(DataInterfaces[GraphEdge.DataInterfaceIndex], GraphEdge.DataInterfaceIndex);

			TCHAR const* WrapNameOverride = GraphEdge.BindingFunctionNameOverride.IsEmpty() ? nullptr : *GraphEdge.BindingFunctionNameOverride;
			TCHAR const* WrapNamespace = GraphEdge.BindingFunctionNamespace.IsEmpty() ? nullptr : *GraphEdge.BindingFunctionNamespace;
			if (GraphEdge.bKernelInput)
			{
				if (ensure(DataInterfaces.IsValidIndex(GraphEdge.DataInterfaceIndex)))
				{
					TArray<FShaderFunctionDefinition> DataProviderFunctions;
					DataInterfaces[GraphEdge.DataInterfaceIndex]->GetSupportedInputs(DataProviderFunctions);
					if (ensure(DataProviderFunctions.IsValidIndex(GraphEdge.DataInterfaceBindingIndex)) && ensure(InKernelSource.ExternalInputs.IsValidIndex(GraphEdge.KernelBindingIndex)))
					{
						FShaderFunctionDefinition const& DataProviderFunction = DataProviderFunctions[GraphEdge.DataInterfaceBindingIndex];
						FShaderFunctionDefinition const& KernelFunction = InKernelSource.ExternalInputs[GraphEdge.KernelBindingIndex];
						GetFunctionShimHLSL(DataProviderFunction, KernelFunction, *NamePrefix, WrapNameOverride, WrapNamespace, HLSL);
					}
				}
			}
			else
			{
				if (ensure(DataInterfaces.IsValidIndex(GraphEdge.DataInterfaceIndex)))
				{
					TArray<FShaderFunctionDefinition> DataProviderFunctions;
					DataInterfaces[GraphEdge.DataInterfaceIndex]->GetSupportedOutputs(DataProviderFunctions);
					if (ensure(DataProviderFunctions.IsValidIndex(GraphEdge.DataInterfaceBindingIndex)) && ensure(InKernelSource.ExternalOutputs.IsValidIndex(GraphEdge.KernelBindingIndex)))
					{
						FShaderFunctionDefinition const&  DataProviderFunction = DataProviderFunctions[GraphEdge.DataInterfaceBindingIndex];
						FShaderFunctionDefinition const& KernelFunction = InKernelSource.ExternalOutputs[GraphEdge.KernelBindingIndex];
						GetFunctionShimHLSL(DataProviderFunction, KernelFunction, *NamePrefix, WrapNameOverride, WrapNamespace, HLSL);
					}
				}
			}
		}
	}

	// Add the kernel code.
	HLSL += InKernelSource.GetSource();

	FString Declaration;
	for (const FString& StructDeclaration : StructDeclarations)
	{
		Declaration += StructDeclaration;
	}

	HLSL = Declaration + HLSL;
	
	// Accumulate the source HLSL to the local hash state.
	HashState.UpdateWithString(*HLSL, HLSL.Len());
	// Finalize hash state and add to the final hash key.
	HashState.Finalize().AppendString(OutHashKey);

	// Add the our boilerplate wrapper to the final hash key.
	GetShaderFileHash(TEXT("/Plugin/ComputeFramework/Private/ComputeKernel.usf"), EShaderPlatform::SP_PCD3D_SM5).AppendString(OutHashKey);

	return HLSL;
}

bool UComputeGraph::CacheCookedResourceShadersForRendering()
{
	for (int32 KernelIndex = 0; KernelIndex < KernelInvocations.Num(); ++KernelIndex)
	{
		UComputeKernel const* Kernel = KernelInvocations[KernelIndex];
		if (Kernel == nullptr || Kernel->KernelSource == nullptr)
		{
			// Skip kernels that would not have been cooked.
			continue;
		}
		if (KernelResources.IsValidIndex(KernelIndex) && KernelResources[KernelIndex].CacheInlineShaders())
		{
			// Found and cached valid inline shader map.
			continue;
		}
		// Some required resource or inline shaders not found.
		return false;
	}
	return true;
}

void UComputeGraph::CacheResourceShadersForRendering(uint32 CompilationFlags)
{
	// Don't build shaders in non-rendering apps such as commandlets.
	if (!FApp::CanEverRender())
	{
		return;
	}

	// Don't build shaders if we are working with cooked data that should contain built inline shader maps.
	// Note that the ContentWorker is a special case that can have cooked resources which don't contain shaders.
	// For that case CacheCookedResourceShadersForRendering() returns false and we will fall back to the compilation path below.
	// Note that this fall back path depends on cooking with optional editor only data so that we can compile shaders using the cooked asset.
	// UOptimusDeformer registers with AutomaticOptionalInclusionAssetType to make that happen. Other usages would need to do the same.
	if (GetPackage()->HasAnyPackageFlags(PKG_Cooked) && CacheCookedResourceShadersForRendering())
	{
		return;
	}

	KernelResources.SetNum(KernelInvocations.Num());

	KernelResourceIndicesPendingShaderCompilation.Reset();
	for (int32 KernelIndex = 0; KernelIndex < KernelInvocations.Num(); ++KernelIndex)
	{
		KernelResourceIndicesPendingShaderCompilation.Add(KernelIndex);
	}
		
	for (int32 KernelIndex = 0; KernelIndex < KernelInvocations.Num(); ++KernelIndex)
	{
		UComputeKernel* Kernel = KernelInvocations[KernelIndex];
		if (Kernel == nullptr || Kernel->KernelSource == nullptr)
		{
			KernelResources[KernelIndex].Reset();
			continue;
		}

		TMap<FString, FString> AdditionalSources = GatherAdditionalSources(Kernel->KernelSource->AdditionalSources);

		FString ShaderHashKey;
		TMap<FString, FString> GeneratedSources;
		TSharedPtr<FComputeKernelDefinitionSet> ShaderDefinitionSet = MakeShared<FComputeKernelDefinitionSet>();
		TSharedPtr<FComputeKernelPermutationVector> ShaderPermutationVector = MakeShared<FComputeKernelPermutationVector>();
		TUniquePtr<FShaderParametersMetadataAllocations> ShaderParameterMetadataAllocations = MakeUnique<FShaderParametersMetadataAllocations>();

		FString ShaderEntryPoint = Kernel->KernelSource->EntryPoint;
		FString ShaderFriendlyName = GetOuter()->GetName() / GetFName().GetPlainNameString() / ShaderEntryPoint;
		FString ShaderSource = BuildKernelSource(KernelIndex, *Kernel->KernelSource, AdditionalSources, ShaderHashKey, GeneratedSources, *ShaderDefinitionSet, *ShaderPermutationVector);
		FShaderParametersMetadata* ShaderParameterMetadata = BuildKernelShaderMetadata(KernelIndex, *ShaderParameterMetadataAllocations);

		const ERHIFeatureLevel::Type CacheFeatureLevel = GMaxRHIFeatureLevel;
		const EShaderPlatform ShaderPlatform = GShaderPlatformForFeatureLevel[CacheFeatureLevel];
		FComputeKernelResource* KernelResource = KernelResources[KernelIndex].GetOrCreate();

		// Now we have all the information that the KernelResource will need for compilation.
		KernelResource->SetupResource(
			CacheFeatureLevel, 
			ShaderFriendlyName,
			ShaderEntryPoint, 
			ShaderHashKey,
			ShaderSource,
			AdditionalSources,
			GeneratedSources,
			ShaderDefinitionSet,
			ShaderPermutationVector,
			ShaderParameterMetadataAllocations,
			ShaderParameterMetadata,
			GetOutermost()->GetFName());

		KernelResource->OnCompilationComplete().BindUObject(this, &UComputeGraph::ShaderCompileCompletionCallback);

		CacheShadersForResource(ShaderPlatform, nullptr, CompilationFlags, KernelResource);
	}
}

void UComputeGraph::CacheShadersForResource(
	EShaderPlatform ShaderPlatform,
	const ITargetPlatform* TargetPlatform,
	uint32 CompilationFlags,
	FComputeKernelResource* KernelResource)
{
	const bool bIsDefault = (KernelResource->GetKernelFlags() & uint32(EComputeKernelFlags::IsDefaultKernel)) != 0;
	if (bIsDefault)
	{
		CompilationFlags |= uint32(EComputeKernelCompilationFlags::Synchronous);
	}

	const bool bIsSuccess = KernelResource->CacheShaders(
		ShaderPlatform,
		TargetPlatform,
		CompilationFlags & uint32(EComputeKernelCompilationFlags::ApplyCompletedShaderMapForRendering),
		CompilationFlags & uint32(EComputeKernelCompilationFlags::Synchronous)
	);

	if (!bIsSuccess)
	{
		if (bIsDefault)
		{
			UE_LOGF(
				LogComputeFramework,
				Fatal,
				"Failed to compile default FComputeKernelResource [%ls] for platform [%ls]!",
				*KernelResource->GetFriendlyName(),
				*LegacyShaderPlatformToShaderFormat(ShaderPlatform).ToString()
			);
		}

		UE_LOGF(
			LogComputeFramework,
			Warning,
			"Failed to compile FComputeKernelResource [%ls] for platform [%ls].",
			*KernelResource->GetFriendlyName(),
			*LegacyShaderPlatformToShaderFormat(ShaderPlatform).ToString()
		);
	}
}

void UComputeGraph::ShaderCompileCompletionCallback(FComputeKernelResource const* KernelResource)
{
	// Find this FComputeKernelResource and call the virtual OnKernelCompilationComplete implementation. 
	for (int32 KernelIndex = 0; KernelIndex < KernelResources.Num(); ++KernelIndex)
	{
		if (KernelResource == KernelResources[KernelIndex].Get())
		{
			OnKernelCompilationComplete(KernelIndex, KernelResource->GetCompilationResults());
			KernelResourceIndicesPendingShaderCompilation.Remove(KernelIndex);
		}
	}
}

void UComputeGraph::BeginCacheForCookedPlatformData(ITargetPlatform const* TargetPlatform)
{
	TArray<FName> ShaderFormats;
	TargetPlatform->GetAllTargetedShaderFormats(ShaderFormats);

	KernelResources.SetNum(KernelInvocations.Num());

	for (int32 KernelIndex = 0; KernelIndex < KernelInvocations.Num(); ++KernelIndex)
	{
		TArray< TUniquePtr<FComputeKernelResource> >& Resources = KernelResources[KernelIndex].CachedKernelResourcesForCooking.FindOrAdd(TargetPlatform);
		Resources.Reset();

		UComputeKernelSource* KernelSource = KernelInvocations[KernelIndex] != nullptr ? KernelInvocations[KernelIndex]->KernelSource : nullptr;
		if (KernelSource == nullptr)
		{
			continue;
		}

		if (ShaderFormats.Num() > 0)
		{
			TMap<FString, FString> AdditionalSources = GatherAdditionalSources(KernelInvocations[KernelIndex]->KernelSource->AdditionalSources);

			FString ShaderHashKey;
			TMap<FString, FString> GeneratedSources;
			TSharedPtr<FComputeKernelDefinitionSet> ShaderDefinitionSet = MakeShared<FComputeKernelDefinitionSet>();
			TSharedPtr<FComputeKernelPermutationVector> ShaderPermutationVector = MakeShared<FComputeKernelPermutationVector>();

			FString ShaderEntryPoint = KernelSource->EntryPoint;
			FString ShaderFriendlyName = GetOuter()->GetName() + TEXT("_") + ShaderEntryPoint;
			FString ShaderSource = BuildKernelSource(KernelIndex, *KernelSource, AdditionalSources, ShaderHashKey, GeneratedSources, *ShaderDefinitionSet, *ShaderPermutationVector);

			for (int32 ShaderFormatIndex = 0; ShaderFormatIndex < ShaderFormats.Num(); ++ShaderFormatIndex)
			{
				TUniquePtr<FShaderParametersMetadataAllocations> ShaderParameterMetadataAllocations = MakeUnique<FShaderParametersMetadataAllocations>();
				FShaderParametersMetadata* ShaderParameterMetadata = BuildKernelShaderMetadata(KernelIndex, *ShaderParameterMetadataAllocations);

				const EShaderPlatform ShaderPlatform = ShaderFormatToLegacyShaderPlatform(ShaderFormats[ShaderFormatIndex]);
				const ERHIFeatureLevel::Type TargetFeatureLevel = GetMaxSupportedFeatureLevel(ShaderPlatform);

				TUniquePtr<FComputeKernelResource> KernelResource = MakeUnique<FComputeKernelResource>();
				KernelResource->SetupResource(
					TargetFeatureLevel, 
					ShaderFriendlyName,
					ShaderEntryPoint, 
					ShaderHashKey,
					ShaderSource,
					AdditionalSources,
					GeneratedSources,
					ShaderDefinitionSet,
					ShaderPermutationVector,
					ShaderParameterMetadataAllocations,
					ShaderParameterMetadata,
					GetOutermost()->GetFName());

				const uint32 CompilationFlags = uint32(EComputeKernelCompilationFlags::Synchronous);
				CacheShadersForResource(ShaderPlatform, TargetPlatform, CompilationFlags, KernelResource.Get());

				Resources.Add(MoveTemp(KernelResource));
			}
		}
	}
}

bool UComputeGraph::IsCachedCookedPlatformDataLoaded(ITargetPlatform const* TargetPlatform)
{
	TArray<FName> ShaderFormats;
	TargetPlatform->GetAllTargetedShaderFormats(ShaderFormats);
	if (ShaderFormats.Num() == 0)
	{
		// Nothing will be queued in BeginCacheForCookedPlatformData.
		return true;
	}

	for (int32 KernelIndex = 0; KernelIndex < KernelInvocations.Num(); ++KernelIndex)
	{
		UComputeKernelSource* KernelSource = KernelInvocations[KernelIndex] != nullptr ? KernelInvocations[KernelIndex]->KernelSource : nullptr;
		if (KernelSource == nullptr)
		{
			continue;
		}

		TArray< TUniquePtr<FComputeKernelResource> >* Resources = KernelResources[KernelIndex].CachedKernelResourcesForCooking.Find(TargetPlatform);
		if (Resources == nullptr)
		{
			return false;
		}

		for (int32 ResourceIndex = 0; ResourceIndex < Resources->Num(); ++ResourceIndex)
		{
			if (!(*Resources)[ResourceIndex]->IsCompilationFinished())
			{
				return false;
			}
		}
	}	

	return true;
}

void UComputeGraph::ClearCachedCookedPlatformData(ITargetPlatform const* TargetPlatform)
{
	for (int32 KernelIndex = 0; KernelIndex < KernelInvocations.Num(); ++KernelIndex)
	{
		KernelResources[KernelIndex].CachedKernelResourcesForCooking.Remove(TargetPlatform);
	}
}

void UComputeGraph::ClearAllCachedCookedPlatformData()
{
	KernelResources.Reset();
}

#endif // WITH_EDITORONLY_DATA

void UComputeGraph::FComputeKernelResourceSet::Reset()
{
#if WITH_EDITORONLY_DATA
	for (int32 FeatureLevel = 0; FeatureLevel < ERHIFeatureLevel::Num; ++FeatureLevel)
	{
		if (KernelResourcesByFeatureLevel[FeatureLevel].IsValid())
		{
			KernelResourcesByFeatureLevel[FeatureLevel]->Invalidate();
			KernelResourcesByFeatureLevel[FeatureLevel] = nullptr;
		}
	}
#else
	if (KernelResource.IsValid())
	{
		KernelResource->Invalidate();
		KernelResource = nullptr;
	}
#endif
}

FComputeKernelResource const* UComputeGraph::FComputeKernelResourceSet::Get() const
{
#if WITH_EDITORONLY_DATA
	ERHIFeatureLevel::Type CacheFeatureLevel = GMaxRHIFeatureLevel;
	return KernelResourcesByFeatureLevel[CacheFeatureLevel].Get();
#else
	return KernelResource.Get();
#endif
}

FComputeKernelResource* UComputeGraph::FComputeKernelResourceSet::GetOrCreate()
{
#if WITH_EDITORONLY_DATA
	ERHIFeatureLevel::Type CacheFeatureLevel = GMaxRHIFeatureLevel;
	if (!KernelResourcesByFeatureLevel[CacheFeatureLevel].IsValid())
	{
		KernelResourcesByFeatureLevel[CacheFeatureLevel] = MakeUnique<FComputeKernelResource>();
	}
	return KernelResourcesByFeatureLevel[CacheFeatureLevel].Get();
#else
	if (!KernelResource.IsValid())
	{
		KernelResource = MakeUnique<FComputeKernelResource>();
	}
	return KernelResource.Get();
#endif
}

void UComputeGraph::FComputeKernelResourceSet::Serialize(FArchive& Ar)
{
#if WITH_EDITORONLY_DATA
	if (Ar.IsSaving())
	{
		int32 NumResourcesToSave = 0;
		const TArray< TUniquePtr<FComputeKernelResource> >* ResourcesToSavePtr = nullptr;

		if (Ar.IsCooking())
		{
			ResourcesToSavePtr = CachedKernelResourcesForCooking.Find(Ar.CookingTarget());
			if (ResourcesToSavePtr != nullptr)
			{
				NumResourcesToSave = ResourcesToSavePtr->Num();
			}
		}

		Ar << NumResourcesToSave;

		if (ResourcesToSavePtr != nullptr)
		{
			for (int32 ResourceIndex = 0; ResourceIndex < ResourcesToSavePtr->Num(); ++ResourceIndex)
			{
				(*ResourcesToSavePtr)[ResourceIndex]->SerializeShaderMap(Ar);
			}
		}
	}
#endif // WITH_EDITORONLY_DATA

	if (Ar.IsLoading())
	{
#if WITH_EDITORONLY_DATA
		const bool HasEditorData = !Ar.IsFilterEditorOnly();
		if (HasEditorData)
		{
			int32 NumLoadedResources = 0;
			Ar << NumLoadedResources;
			for (int32 i = 0; i < NumLoadedResources; i++)
			{
				TUniquePtr<FComputeKernelResource> LoadedResource = MakeUnique<FComputeKernelResource>();
				LoadedResource->SerializeShaderMap(Ar);
				LoadedKernelResources.Add(MoveTemp(LoadedResource));
			}
		}
		else
#endif // WITH_EDITORONLY_DATA
		{
			int32 NumLoadedResources = 0;
			Ar << NumLoadedResources;
			for (int32 ResourceIndex = 0; ResourceIndex < NumLoadedResources; ++ResourceIndex)
			{
				TUniquePtr<FComputeKernelResource> Resource = MakeUnique<FComputeKernelResource>();
				Resource->SerializeShaderMap(Ar);

				FComputeKernelShaderMap* ShaderMap = Resource->GetGameThreadShaderMap();
				if (ShaderMap != nullptr)
				{
					if (GMaxRHIShaderPlatform == ShaderMap->GetShaderPlatform())
					{
#if WITH_EDITORONLY_DATA
						KernelResourcesByFeatureLevel[GMaxRHIFeatureLevel] = MoveTemp(Resource);
#else
						KernelResource = MoveTemp(Resource);
#endif
					}
				}
			}
		}
	}
}

void UComputeGraph::FComputeKernelResourceSet::ProcessSerializedShaderMaps()
{
#if WITH_EDITORONLY_DATA
	for (TUniquePtr<FComputeKernelResource>& LoadedResource : LoadedKernelResources)
	{
		FComputeKernelShaderMap* LoadedShaderMap = LoadedResource->GetGameThreadShaderMap();
		if (LoadedShaderMap && LoadedShaderMap->GetShaderPlatform() == GMaxRHIShaderPlatform)
		{
			ERHIFeatureLevel::Type LoadedFeatureLevel = LoadedShaderMap->GetShaderMapId().FeatureLevel;
			if (!KernelResourcesByFeatureLevel[LoadedFeatureLevel].IsValid())
			{
				KernelResourcesByFeatureLevel[LoadedFeatureLevel] = MakeUnique<FComputeKernelResource>();
			}

			KernelResourcesByFeatureLevel[LoadedFeatureLevel]->SetInlineShaderMap(LoadedShaderMap);
		}
		else
		{
			LoadedResource->DiscardShaderMap();
		}
	}

	LoadedKernelResources.Reset();
#endif
}

bool UComputeGraph::FComputeKernelResourceSet::CacheInlineShaders()
{
#if WITH_EDITORONLY_DATA
	const ERHIFeatureLevel::Type CacheFeatureLevel = GMaxRHIFeatureLevel;
	if (FComputeKernelResource* KernelResource = KernelResourcesByFeatureLevel[CacheFeatureLevel].Get())
	{
		const EShaderPlatform ShaderPlatform = GShaderPlatformForFeatureLevel[CacheFeatureLevel];
		return KernelResource->CacheInlineShaders(ShaderPlatform);
	}
#endif
	return false;
}

#if WITH_EDITORONLY_DATA
void UComputeGraph::FComputeKernelResourceSet::CancelCompilation()
{
	for (int32 FeatureLevel = 0; FeatureLevel < ERHIFeatureLevel::Num; ++FeatureLevel)
	{
		if (KernelResourcesByFeatureLevel[FeatureLevel].IsValid())
		{
			KernelResourcesByFeatureLevel[FeatureLevel]->CancelCompilation();
		}
	}
}
#endif
