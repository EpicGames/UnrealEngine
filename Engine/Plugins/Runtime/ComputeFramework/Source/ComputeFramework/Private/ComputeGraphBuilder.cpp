// Copyright Epic Games, Inc. All Rights Reserved.

#include "ComputeFramework/ComputeGraphBuilder.h"

#include "Algo/Count.h"
#include "ComputeFramework/ComputeDataInterface.h"
#include "ComputeFramework/ComputeGraph.h"
#include "ComputeFramework/ComputeKernel.h"
#include "ComputeFramework/ComputeKernelSource.h"
#include "ComputeFramework/ShaderParamTypeDefinition.h"

#if WITH_EDITOR

FKernelHandle FComputeGraphBuilder::AddKernel(UComputeKernelSource* Source, FName Name)
{
	if (!Source)
	{
		return FKernelHandle{};
	}

	FKernelEntry& Entry = KernelEntries.AddDefaulted_GetRef();
	Entry.Source = Source;
	Entry.Name = Name;
	return FKernelHandle{ KernelEntries.Num() - 1 };
}

FInterfaceHandle FComputeGraphBuilder::AddDataInterface(UComputeDataInterface* Interface, UClass* BindingClass, FName Name)
{
	if (!Interface || !BindingClass)
	{
		return FInterfaceHandle{};
	}

	FInterfaceEntry& Entry = InterfaceEntries.AddDefaulted_GetRef();
	Entry.Interface = Interface;
	Entry.BindingClass = BindingClass;
	Entry.Name = Name;
	return FInterfaceHandle{ InterfaceEntries.Num() - 1 };
}

void FComputeGraphBuilder::ConnectInput(FKernelHandle Kernel, FStringView KernelFn, FInterfaceHandle Interface, FStringView InterfaceFn)
{
	if (!KernelEntries.IsValidIndex(Kernel.Index) || !InterfaceEntries.IsValidIndex(Interface.Index) || KernelFn.IsEmpty() || InterfaceFn.IsEmpty())
	{
		return;
	}

	FPinEntry& Pin = KernelEntries[Kernel.Index].Inputs.AddDefaulted_GetRef();
	Pin.KernelFunctionName = KernelFn;
	Pin.InterfaceIndex = Interface.Index;
	Pin.InterfaceFunctionName = InterfaceFn;
}

void FComputeGraphBuilder::ConnectOutput(FKernelHandle Kernel, FStringView KernelFn, FInterfaceHandle Interface, FStringView InterfaceFn)
{
	if (!KernelEntries.IsValidIndex(Kernel.Index) || !InterfaceEntries.IsValidIndex(Interface.Index) || KernelFn.IsEmpty() || InterfaceFn.IsEmpty())
	{
		return;
	}

	FPinEntry& Pin = KernelEntries[Kernel.Index].Outputs.AddDefaulted_GetRef();
	Pin.KernelFunctionName = KernelFn;
	Pin.InterfaceIndex = Interface.Index;
	Pin.InterfaceFunctionName = InterfaceFn;
}

void FComputeGraphBuilder::Build(UComputeGraph& OutGraph, UObject& Outer) const
{
	OutGraph.KernelInvocations.Reset();
	OutGraph.DataInterfaces.Reset();
	OutGraph.GraphEdges.Reset();
	OutGraph.Bindings.Reset();
	OutGraph.DataInterfaceToBinding.Reset();

	// Register unique binding classes, preserving insertion order.
	TMap<UClass*, int32> BindingClassToIndex;
	for (FInterfaceEntry const& Entry : InterfaceEntries)
	{
		if (!BindingClassToIndex.Contains(Entry.BindingClass))
		{
			BindingClassToIndex.Add(Entry.BindingClass, OutGraph.Bindings.Add(Entry.BindingClass));
		}
	}

	// Add data interfaces.
	for (FInterfaceEntry const& Entry : InterfaceEntries)
	{
		OutGraph.DataInterfaces.Add(Entry.Interface);
		OutGraph.DataInterfaceToBinding.Add(BindingClassToIndex[Entry.BindingClass]);
	}

	// Add kernels and wire their pins.
	for (FKernelEntry const& KernelEntry : KernelEntries)
	{
		UComputeKernel* Kernel = NewObject<UComputeKernel>(&Outer);
		Kernel->KernelSource = KernelEntry.Source;
		const int32 KernelIndex = OutGraph.KernelInvocations.Add(Kernel);

		auto AddConnection = [&](FPinEntry const& Pin, bool const bInput)
		{
			if (!OutGraph.DataInterfaces.IsValidIndex(Pin.InterfaceIndex))
			{
				return;
			}

			TArray<FShaderFunctionDefinition> Functions;
			if (bInput)
			{
				OutGraph.DataInterfaces[Pin.InterfaceIndex]->GetSupportedInputs(Functions);
			}
			else
			{
				OutGraph.DataInterfaces[Pin.InterfaceIndex]->GetSupportedOutputs(Functions);
			}

			const int32 FunctionIndex = Functions.IndexOfByPredicate([&Pin](FShaderFunctionDefinition const& Def) { return Def.Name == Pin.InterfaceFunctionName; });
			if (FunctionIndex == INDEX_NONE)
			{
				return;
			}

			if (bInput)
			{
				Kernel->KernelSource->ExternalInputs.Add_GetRef(Functions[FunctionIndex]).SetName(Pin.KernelFunctionName);
			}
			else
			{
				Kernel->KernelSource->ExternalOutputs.Add_GetRef(Functions[FunctionIndex]).SetName(Pin.KernelFunctionName);
			}

			const int32 KernelBindingIndex = Algo::CountIf(OutGraph.GraphEdges,	[KernelIndex, bInput](FComputeGraphEdge const& Edge)
			{
				return Edge.KernelIndex == KernelIndex && Edge.bKernelInput == bInput;
			});

			FComputeGraphEdge& Edge = OutGraph.GraphEdges.AddDefaulted_GetRef();
			Edge.KernelIndex = KernelIndex;
			Edge.KernelBindingIndex = KernelBindingIndex;
			Edge.DataInterfaceIndex = Pin.InterfaceIndex;
			Edge.DataInterfaceBindingIndex = FunctionIndex;
			Edge.bKernelInput = bInput;
		};

		Kernel->KernelSource->ExternalInputs.Reset();
		for (FPinEntry const& Pin : KernelEntry.Inputs)
		{
			AddConnection(Pin, true);
		}

		Kernel->KernelSource->ExternalOutputs.Reset();
		for (FPinEntry const& Pin : KernelEntry.Outputs)
		{ 
			AddConnection(Pin, false);
		}
	}
}


#endif // WITH_EDITOR
