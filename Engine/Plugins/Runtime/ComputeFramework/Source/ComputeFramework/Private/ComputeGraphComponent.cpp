// Copyright Epic Games, Inc. All Rights Reserved.

#include "ComputeFramework/ComputeGraphComponent.h"

#include "ComputeFramework/ComputeFramework.h"
#include "ComputeFramework/ComputeDataProvider.h"
#include "ComputeFramework/ComputeGraph.h"
#include "ComputeFramework/ComputeGraphInstance.h"
#include "ComputeWorkerInterface.h"
#include "GameFramework/Actor.h"
#include "UnrealEngine.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ComputeGraphComponent)

UComputeGraphComponent::UComputeGraphComponent()
{
	PrimaryComponentTick.bCanEverTick = true;

	// By default don't tick and allow any queuing of work to be handled by blueprint.
	// Ticking can be turned on by some systems that need it (such as editor window).
	PrimaryComponentTick.bStartWithTickEnabled = false;
}

void UComputeGraphComponent::OnRegister()
{
	Super::OnRegister();

	if (ComputeGraph != nullptr)
	{
		const int32 NumBindingObjects = ComputeGraph->GetNumBindingObjects();
		for (int32 BindingObjectIndex = 0; BindingObjectIndex < NumBindingObjects; ++BindingObjectIndex)
		{
			ComputeGraphInstance.CreateDataProviders(ComputeGraph, BindingObjectIndex, GetBindingObject(BindingObjectIndex));
		}
	}
}

void UComputeGraphComponent::OnUnregister()
{
	ComputeGraphInstance.DestroyDataProviders();
	
	Super::OnUnregister();
}

void UComputeGraphComponent::InitializeProvider_Implementation(int32 InDataInterfaceIndex, UObject* InOutDataProvider)
{
	if (InDataInterfaceIndex < 0)
	{
		return;
	}
	
	UComputeDataProvider* Provider = Cast<UComputeDataProvider>(InOutDataProvider);
	if (Provider == nullptr)
	{
		return;
	}

	if (!DataProviderTemplates.IsValidIndex(InDataInterfaceIndex))
	{ 
		DataProviderTemplates.SetNum(InDataInterfaceIndex + 1);
	}
	if (DataProviderTemplates[InDataInterfaceIndex] == nullptr)
	{
		DataProviderTemplates[InDataInterfaceIndex] = DuplicateObject<UComputeDataProvider>(Provider, this);
	}
	else if (DataProviderTemplates[InDataInterfaceIndex]->GetClass() == Provider->GetClass())
	{
		UEngine::CopyPropertiesForUnrelatedObjects(DataProviderTemplates[InDataInterfaceIndex], Provider);
	}
}

UObject* UComputeGraphComponent::GetBindingObject_Implementation(int32 InBindingIndex)
{
	// By default use this as the binding object.
	return this;
}

void UComputeGraphComponent::QueueExecute()
{
	if (ComputeGraph != nullptr)
	{
		MarkRenderDynamicDataDirty();
	}
}

void UComputeGraphComponent::SendRenderDynamicData_Concurrent()
{
	Super::SendRenderDynamicData_Concurrent();

	ComputeGraphInstance.EnqueueWork(ComputeGraph, GetScene(), ComputeTaskExecutionGroup::EndOfFrameUpdate, GetOwner()->GetFName(), FSimpleDelegate(), this);
}

void UComputeGraphComponent::DestroyRenderState_Concurrent()
{
	Super::DestroyRenderState_Concurrent();
	
	ComputeFramework::AbortWork(GetScene(), this);
}
