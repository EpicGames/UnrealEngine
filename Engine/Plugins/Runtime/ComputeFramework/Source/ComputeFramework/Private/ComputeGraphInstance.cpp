// Copyright Epic Games, Inc. All Rights Reserved.

#include "ComputeFramework/ComputeGraphInstance.h"

#include "ComputeFramework/ComputeDataInterface.h"
#include "ComputeFramework/ComputeDataProvider.h"
#include "ComputeFramework/ComputeFramework.h"
#include "ComputeFramework/ComputeFrameworkModule.h"
#include "ComputeFramework/ComputeGraph.h"
#include "ComputeFramework/ComputeGraphRenderProxy.h"
#include "ComputeFramework/ComputeGraphWorker.h"
#include "ComputeFramework/ComputeSystem.h"
#include "RenderingThread.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ComputeGraphInstance)

void FComputeGraphInstance::CreateDataProviders(UComputeGraph* InComputeGraph, int32 InBindingIndex, TObjectPtr<UObject> InBindingObject)
{
	if (InComputeGraph != nullptr)
	{
		InComputeGraph->CreateDataProviders(InBindingIndex, InBindingObject, DataProviders);
	}
}

void FComputeGraphInstance::InitializeDataProviders(UComputeGraph* InComputeGraph, int32 InBindingIndex, TObjectPtr<UObject> InBindingObject)
{
	if (InComputeGraph != nullptr)
	{
		InComputeGraph->InitializeDataProviders(InBindingIndex, InBindingObject, DataProviders);
	}
}

void FComputeGraphInstance::ResetDataProviders()
{
	for (UComputeDataProvider* DataProvider : DataProviders)
	{
		if (DataProvider)
		{
			DataProvider->Reset();
		}
	}
}

void FComputeGraphInstance::DestroyDataProviders()
{
	DataProviders.Reset();
}

bool FComputeGraphInstance::EnqueueWork(
	UComputeGraph* InComputeGraph,
	FSceneInterface const* InScene,
	FName InExecutionGroupName,
	FName InOwnerName,
	FSimpleDelegate InFallbackDelegate,
	UObject* InOwnerPointer,
	uint8 InGraphSortPriorityOffset
#if HAS_GPU_STATS
	, TArray<TPromise<FComputeFrameworkKernelTiming>> InKernelTimingPromises
#endif
	)
{
	if (InComputeGraph == nullptr || InScene == nullptr)
	{
		return false;
	}

	if (!ComputeFramework::IsEnabled() || FComputeFrameworkModule::GetComputeSystem() == nullptr)
	{
		return false;
	}

	// Don't submit work if we don't have all of the expected bindings.
	if (!InComputeGraph->ValidateProviders(DataProviders))
	{
		return false;
	}

	// Lookup the compute worker associated with this scene.
	FComputeGraphTaskWorker* ComputeGraphWorker = FComputeFrameworkModule::GetComputeSystem()->GetComputeWorker(InScene);
	if (!ensure(ComputeGraphWorker))
	{
		return false;
	}

	FComputeGraphRenderProxy const* GraphRenderProxy = InComputeGraph->GetRenderProxy();
	if (GraphRenderProxy == nullptr)
	{
		// This can happen if we have deferred compilation.
		// Trigger compilation now.
		ensure(ComputeFramework::IsDeferredCompilation());

		InComputeGraph->UpdateResources();
		return false;
	}

	TArray<FComputeDataProviderRenderProxy*> DataProviderRenderProxies;
	for (UComputeDataProvider* DataProvider : DataProviders)
	{
		// Be sure to add null provider slots because we want to maintain consistent array indices.
		// Note that we expect GetRenderProxy() to return a pointer that we can own and call delete on.
		FComputeDataProviderRenderProxy* ProviderProxy = DataProvider != nullptr ? DataProvider->GetRenderProxy() : nullptr;
		DataProviderRenderProxies.Add(ProviderProxy);
	}

#if HAS_GPU_STATS
	// GPU profiler callbacks only fire at real frame boundaries; in automation tests none exist. Fulfill promises immediately on the game thread
	// so callers don't wait, and pass an empty array to the render command so no FComputeFrameworkGPUStat objects are created.
	if (GIsAutomationTesting)
	{
		for (TPromise<FComputeFrameworkKernelTiming>& Promise : InKernelTimingPromises)
		{
			Promise.SetValue(FComputeFrameworkKernelTiming{});
		}
		InKernelTimingPromises.Reset();
	}
#endif

	ENQUEUE_RENDER_COMMAND(ComputeFrameworkEnqueueExecutionCommand)(
		[ComputeGraphWorker, InExecutionGroupName, InOwnerName, SortPriority = InGraphSortPriorityOffset + GraphSortPriority, GraphRenderProxy, MovedDataProviderRenderProxies = MoveTemp(DataProviderRenderProxies), InFallbackDelegate, InOwnerPointer, bEnableRenderCaptures=this->bEnableRenderCaptures
#if HAS_GPU_STATS
		, MovedKernelTimingPromises = MoveTemp(InKernelTimingPromises)
#endif
		](FRHICommandListImmediate& RHICmdList) mutable
		{
			// Compute graph scheduler will take ownership of the provider proxies.
			ComputeGraphWorker->Enqueue(InExecutionGroupName, InOwnerName, SortPriority, GraphRenderProxy, MovedDataProviderRenderProxies, InFallbackDelegate, bEnableRenderCaptures
#if HAS_GPU_STATS
				, MoveTemp(MovedKernelTimingPromises)
#endif
				, InOwnerPointer);
		});

	return true;
}
