// Copyright Epic Games, Inc. All Rights Reserved.

#include "ComputeFramework/ComputeGraphWorker.h"

#include "Algo/Sort.h"
#include "ComputeFramework/ComputeKernel.h"
#include "ComputeFramework/ComputeKernelPermutationVector.h"
#include "ComputeFramework/ComputeKernelShader.h"
#include "ComputeFramework/ComputeGraph.h"
#include "ComputeFramework/ComputeGraphRenderProxy.h"
#include "ProfilingDebugging/RealtimeGPUProfiler.h"
#include "RenderCaptureInterface.h"
#include "GPUProfiler.h"
#include "RenderGraphBuilder.h"
#include "RenderGraphUtils.h"
#include "RHIBreadcrumbs.h"
#include "RHIDefinitions.h"
#include "RHIGPUReadback.h"
#include "SkeletalMeshUpdater.h"

#if HAS_GPU_STATS
/** Per-kernel GPU stat created by FComputeGraphTaskWorker::SubmitWork.
 *
 *  Lifetime: allocated with 'new' just before the RDG event scope is opened. OnTimingResults returns EOnTimingResultsAction::Delete,
 *  causing ProcessFrame to collect and delete the object after all per-queue callbacks for the frame have fired. The corresponding
 *  TFuture is held by client code which can poll it until ready. */
struct FComputeFrameworkGPUStat : public UE::RHI::GPUProfiler::FGPUStat
{
    TPromise<FComputeFrameworkKernelTiming> Promise;

    explicit FComputeFrameworkGPUStat(TPromise<FComputeFrameworkKernelTiming>&& InPromise)
        : FGPUStat(TEXT("ComputeKernel"), TEXT("Compute Kernel"), /*DrawStatsCategory=*/nullptr)
        , Promise(MoveTemp(InPromise))
    {
        bEmitToEngineStats = false;
    }

    virtual EOnTimingResultsAction OnTimingResults(UE::RHI::GPUProfiler::FQueue Queue, double BusyMs, double /*IdleMs*/, double /*WaitMs*/) override
    {
		// Each stat is only submitted to a single queue, so we are ignoring the queue type and index.
		// TODO: Test and handle multi-GPU - for now forward timing data from first GPU.
		if (Queue.GPU == 0)
		{
			Promise.SetValue(FComputeFrameworkKernelTiming{ BusyMs });
		}

		// Mark this stat for deletion as this stat is per-dispatch and should not persist across frames.
		return EOnTimingResultsAction::Delete;
    }
};
#endif // HAS_GPU_STATS

DECLARE_GPU_STAT_NAMED(ComputeFramework_ExecuteBatches, TEXT("ComputeFramework::ExecuteBatches"));

static TAutoConsoleVariable<int32> CVarComputeFrameworkSortSubmit(
	TEXT("r.ComputeFramework.SortSubmit"),
	1,
	TEXT("Sort submission of work to GPU for optimal scheduling."),
	ECVF_RenderThreadSafe
);

static int32 GTriggerGPUCaptureDispatches = 0;
static FAutoConsoleVariableRef CVarComputeFrameworkTriggerGPUCapture(
	TEXT("r.ComputeFramework.TriggerGPUCaptureDispatches"),
	GTriggerGPUCaptureDispatches,
	TEXT("Trigger GPU captures for this many of the subsequent compute graph dispatches."),
	ECVF_RenderThreadSafe
);


void FComputeGraphTaskWorker::Enqueue(
	FName InExecutionGroupName, 
	FName InOwnerName, 
	uint8 InGraphSortPriority,
	FComputeGraphRenderProxy const* InGraphRenderProxy, 
	TArray<FComputeDataProviderRenderProxy*> InDataProviderRenderProxies, 
	FSimpleDelegate InFallbackDelegate,
	bool bInEnableRenderCaptures,
#if HAS_GPU_STATS
	TArray<TPromise<FComputeFrameworkKernelTiming>> InKernelTimingPromises,
#endif
	const UObject* InOwnerPointer)
{
	FGraphInvocation& GraphInvocation = GraphInvocationsPerGroup.FindOrAdd(InExecutionGroupName).AddDefaulted_GetRef();
	GraphInvocation.OwnerName = InOwnerName;
	GraphInvocation.OwnerPointer = InOwnerPointer;
	GraphInvocation.GraphSortPriority = InGraphSortPriority;
	GraphInvocation.GraphRenderProxy = InGraphRenderProxy;
	GraphInvocation.DataProviderRenderProxies = MoveTemp(InDataProviderRenderProxies);
	GraphInvocation.FallbackDelegate = InFallbackDelegate;
	GraphInvocation.bEnableRenderCaptures = bInEnableRenderCaptures;
#if HAS_GPU_STATS
	GraphInvocation.KernelTimingPromises = MoveTemp(InKernelTimingPromises);
#endif
}

void FComputeGraphTaskWorker::Abort(const UObject* InOwnerPointer)
{
	for (auto& Pair : GraphInvocationsPerGroup)
	{
		TArray<FGraphInvocation>& Invocations = Pair.Value;

		for (int32 Index = Invocations.Num() - 1; Index >= 0; Index--)
		{
            if (Invocations[Index].OwnerPointer == InOwnerPointer)
            {
#if HAS_GPU_STATS
                Invocations[Index].SendNullKernelTimings();
#endif

                Invocations.RemoveAt(Index, 1, EAllowShrinking::No);
            }
		}
	}
	
	// Not clearing ActiveAsyncReadbacks here because the system requesting 
	// the readback data isn't always InOwnerPointer, instead it can be some external system that
	// simply wants to readback the latest data produced by InOwnerPointer.
	
	// When InOwnerPointer becomes invalid at a frame, any readback requested in the previous frame 
	// can still be useful to the requester, but it is likely not ready until 2 or 3 frames later.
	// So the worker needs to keep polling for that last bit of data for a few extra frames.
	
	// The Requester system can use OnDataAvailable callback to decide if it actually wants
	// to use the data or discard it.
	
	// For example, when baking level sequence to geometry cache, the baker can request a readback right before the frame 
	// where the skeletal mesh despawns. The data for that frame won't be ready until 2 or 3 frames after 
	// skeletal mesh has been destroyed, so we need the worker to keep polling for that data for 2-3 extra frames.
	
}

bool FComputeGraphTaskWorker::HasWork(FName InExecutionGroupName) const
{
	// Currently poll readbacks once at end of frame.
	if (!ActiveAsyncReadbacks.IsEmpty() && InExecutionGroupName == ComputeTaskExecutionGroup::EndOfFrameUpdate)
	{
		return true;
	}

	TArray<FGraphInvocation> const* GraphInvocations = GraphInvocationsPerGroup.Find(InExecutionGroupName);
	return GraphInvocations != nullptr && GraphInvocations->Num();
}

void FComputeGraphTaskWorker::SubmitWork(FComputeContext& Context)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(ComputeFramework::ExecuteBatches);
	RDG_EVENT_SCOPE_STAT(Context.GraphBuilder, ComputeFramework_ExecuteBatches, "ComputeFramework::ExecuteBatches");

	FRDGBuilder& GraphBuilder = Context.GraphBuilder;

	const bool bCaptureAllWork = GTriggerGPUCaptureDispatches > 0;
	RenderCaptureInterface::FScopedCapture RenderCapture(bCaptureAllWork, GraphBuilder, TEXT("FComputeGraphTaskWorker::SubmitWork"));
	GTriggerGPUCaptureDispatches = FMath::Max(GTriggerGPUCaptureDispatches - 1, 0);

	// Currently poll readbacks once at end of frame.
	if (Context.ExecutionGroupName == ComputeTaskExecutionGroup::EndOfFrameUpdate)
	{
		UpdateReadbacks();
	}

	TArray<FGraphInvocation> const* FoundGraphInvocations = GraphInvocationsPerGroup.Find(Context.ExecutionGroupName);
	if (!FoundGraphInvocations || FoundGraphInvocations->Num() == 0)
	{
		return;
	}

	// Reset our scratch memory arrays.
	SubmitDescs.Reset();
	Shaders.Reset();

	// Sync mesh deformer updater tasks so that calls to FSkeletalMeshDeformerHelpers can access data.
	FSkeletalMeshUpdater::WaitForStage(GraphBuilder, ESkeletalMeshUpdateStage::MeshDeformer);

	FRDGExternalAccessQueue ExternalAccessQueue;

	TArray<FGraphInvocation> const& GraphInvocations = *FoundGraphInvocations;
	for (int32 GraphIndex = 0; GraphIndex < GraphInvocations.Num(); ++GraphIndex)
	{
		FGraphInvocation const& GraphInvocation = GraphInvocations[GraphIndex];
		FComputeGraphRenderProxy const* GraphRenderProxy = GraphInvocation.GraphRenderProxy;
		const int32 NumKernels = GraphRenderProxy->KernelInvocations.Num();

		const int32 BaseSubmitDescIndex = SubmitDescs.Num();
		SubmitDescs.Reserve(BaseSubmitDescIndex + NumKernels);
		const int32 BaseShaderIndex = Shaders.Num();

		// Gather shaders and validate the DataInterfaces.
		// If validation fails or shaders are awaiting compilation we will not run the graph.
		bool bIsValid = true;
		for (int32 KernelIndex = 0; bIsValid && KernelIndex < NumKernels; ++KernelIndex)
		{
			FSubmitDescription& SubmitDesc = SubmitDescs.AddZeroed_GetRef();
			SubmitDesc.GraphIndex = GraphIndex;
			SubmitDesc.KernelIndex = KernelIndex;
			SubmitDesc.GraphSortPriority = GraphInvocation.GraphSortPriority;
			SubmitDesc.ShaderIndex = Shaders.Num();

			FComputeGraphRenderProxy::FKernelInvocation const& KernelInvocation = GraphRenderProxy->KernelInvocations[KernelIndex];
			SubmitDesc.bTriggerRenderCapture = GraphInvocation.bEnableRenderCaptures && KernelInvocation.bTriggerRenderCapture;

			// Reset our scratch memory arrays.
			PermutationIds.Reset();
			ThreadCounts.Reset();

			const int32 NumSubInvocations = GraphInvocation.DataProviderRenderProxies[KernelInvocation.ExecutionProviderIndex]->GetDispatchThreadCount(ThreadCounts);

			// Iterate shader parameter members to fill the dispatch data structures.
			// We assume that the members were filled out with a single data interface per member, and that the
			// order is the same one defined in the KernelInvocation.BoundProviderIndices.
			TArray<FShaderParametersMetadata::FMember> const& ParamMembers = KernelInvocation.ShaderParameterMetadata->GetMembers();

			FComputeDataProviderRenderProxy::FPermutationData PermutationData{ NumSubInvocations, GraphRenderProxy->ShaderPermutationVectors[KernelIndex], MoveTemp(PermutationIds) };
			PermutationData.PermutationIds.SetNumZeroed(NumSubInvocations);

			for (int32 MemberIndex = 0; bIsValid && MemberIndex < ParamMembers.Num(); ++MemberIndex)
			{
				FShaderParametersMetadata::FMember const& Member = ParamMembers[MemberIndex];
				if (ensure(Member.GetBaseType() == EUniformBufferBaseType::UBMT_NESTED_STRUCT))
				{
					const int32 DataProviderIndex = KernelInvocation.BoundProviderIndices[MemberIndex];
					FComputeDataProviderRenderProxy* DataProvider = GraphInvocation.DataProviderRenderProxies[DataProviderIndex];
					if (ensure(DataProvider != nullptr))
					{
						FComputeDataProviderRenderProxy::FValidationData ValidationData{ NumSubInvocations, (int32)Member.GetStructMetadata()->GetSize() };
						bIsValid &= DataProvider->IsValid(ValidationData);

						if (!bIsValid)
						{
							continue;
						}
						
						DataProvider->GatherPermutations(PermutationData);
					}
				}
			}

			// Get shader. This can fail if compilation is pending.
			for (int32 SubInvocationIndex = 0; bIsValid && SubInvocationIndex < NumSubInvocations; ++SubInvocationIndex)
			{
				TShaderRef<FComputeKernelShader> Shader = KernelInvocation.KernelResource->GetShader(PermutationData.PermutationIds[SubInvocationIndex]);
				bIsValid &= Shader.IsValid();
				Shaders.Add(Shader);
			}

			// Check if we can do unified dispatch and apply that if we can.
			if (bIsValid && KernelInvocation.bSupportsUnifiedDispatch && NumSubInvocations > 1)
			{
				bool bSupportsUnifiedDispatch = true;
				for (int32 SubInvocationIndex = 1; bSupportsUnifiedDispatch && SubInvocationIndex < NumSubInvocations; ++SubInvocationIndex)
				{
					bSupportsUnifiedDispatch &= Shaders[SubmitDesc.ShaderIndex + SubInvocationIndex] == Shaders[SubmitDesc.ShaderIndex];
				}

				if (bSupportsUnifiedDispatch)
				{
					SubmitDesc.bIsUnified = true;
					Shaders.SetNum(SubmitDesc.ShaderIndex + 1, EAllowShrinking::No);
				}
			}

			// Move our scratch array back for subsequent reuse.
			PermutationIds = MoveTemp(PermutationData.PermutationIds);
		}

		// If we can't run the graph for any reason, back out now and apply fallback logic.
		if (!bIsValid)
		{
			SubmitDescs.SetNum(BaseSubmitDescIndex, EAllowShrinking::No);
			Shaders.SetNum(BaseShaderIndex, EAllowShrinking::No);

#if HAS_GPU_STATS
			GraphInvocation.SendNullKernelTimings();
#endif

			GraphInvocation.FallbackDelegate.ExecuteIfBound();
			continue;
		}

		// Allocate RDG resources for all the data providers in the graph.
		FComputeDataProviderRenderProxy::FAllocationData AllocationData { NumKernels, ExternalAccessQueue };
		for (int32 DataProviderIndex = 0; DataProviderIndex < GraphInvocation.DataProviderRenderProxies.Num(); ++DataProviderIndex)
		{
			FComputeDataProviderRenderProxy* DataProvider = GraphInvocation.DataProviderRenderProxies[DataProviderIndex];
			if (DataProvider != nullptr)
			{
				DataProvider->AllocateResources(GraphBuilder, AllocationData);
			}
		}
	}

	if (CVarComputeFrameworkSortSubmit.GetValueOnRenderThread() != 0)
	{
		// Sort for optimal dispatch.
		Algo::Sort(SubmitDescs, [](const FSubmitDescription& LHS, const FSubmitDescription& RHS) { return LHS.PackedSortKey < RHS.PackedSortKey; });
	}

	for (FSubmitDescription const& SubmitDesc : SubmitDescs)
	{
		const int32 GraphIndex = SubmitDesc.GraphIndex;
		FGraphInvocation const& GraphInvocation = GraphInvocations[GraphIndex];
		FComputeGraphRenderProxy const* GraphRenderProxy = GraphInvocation.GraphRenderProxy;

		const int32 KernelIndex = SubmitDesc.KernelIndex;
		FComputeGraphRenderProxy::FKernelInvocation const& KernelInvocation = GraphRenderProxy->KernelInvocations[KernelIndex];

		RenderCaptureInterface::FScopedCapture SubmitRenderCapture(SubmitDesc.bTriggerRenderCapture && !bCaptureAllWork, GraphBuilder, TEXT("FComputeGraphTaskWorker::SubmitWork::SubmitDesc"));

		// Open an RDG event scope for this kernel. Three configs:
		//   HAS_GPU_STATS + WITH_RHI_BREADCRUMBS_FULL: stat-tracked scope with full "Owner:Graph:Kernel" name string.
		//   HAS_GPU_STATS only: stat-tracked scope, name string omitted (breadcrumbs not available).
		//   neither: plain event scope with full name string, no stat tracking.
#if HAS_GPU_STATS
		// Create a stat from the promise. The GPUProfiler takes ownership of the object and deletes it after all per-queue callbacks for the frame have fired.
		UE::RHI::GPUProfiler::FGPUStat* KernelGPUStat = nullptr;
		if (GraphInvocation.KernelTimingPromises.IsValidIndex(KernelIndex))
		{
			KernelGPUStat = new FComputeFrameworkGPUStat(MoveTemp(GraphInvocation.KernelTimingPromises[KernelIndex]));
		}
		const ERDGScopeFlags KernelScopeFlags = KernelGPUStat ? ERDGScopeFlags::Stat : ERDGScopeFlags::None;

#if WITH_RHI_BREADCRUMBS_FULL
		RDG_EVENT_SCOPE_IMPL(GraphBuilder, true, KernelScopeFlags, KernelGPUStat, TEXT("ComputeKernel"), RDG_SCOPE_ARGS("%s:%s:%s", *GraphInvocation.OwnerName.ToString(), *GraphRenderProxy->GraphName.ToString(), *KernelInvocation.KernelName));
#else
		RDG_EVENT_SCOPE_IMPL(GraphBuilder, true, KernelScopeFlags, KernelGPUStat, TEXT("ComputeKernel"), nullptr);
#endif
#else
		RDG_EVENT_SCOPE(GraphBuilder, "%s:%s:%s", *GraphInvocation.OwnerName.ToString(), *GraphRenderProxy->GraphName.ToString(), *KernelInvocation.KernelName);
#endif

		Context.GraphKernelIndex = KernelIndex;

		// Pre submit calls.
		for (int32 DataProviderIndex : KernelInvocation.PreSubmitProviderIndices)
		{
			if (FComputeDataProviderRenderProxy* DataProviderProxy = GraphInvocation.DataProviderRenderProxies[DataProviderIndex])
			{
				DataProviderProxy->PreSubmit(Context);
			}
		}

		//todo[CF]: GetDispatchThreadCount() should take the bIsUnified flag directly.
		ThreadCounts.Reset();
		int32 NumSubInvocations = GraphInvocation.DataProviderRenderProxies[KernelInvocation.ExecutionProviderIndex]->GetDispatchThreadCount(ThreadCounts);

		bool bIsUnifiedDispatch = SubmitDesc.bIsUnified;
		if (bIsUnifiedDispatch)
		{
			for (int32 SubInvocationIndex = 1; SubInvocationIndex < NumSubInvocations; ++SubInvocationIndex)
			{
				ThreadCounts[0].X += ThreadCounts[SubInvocationIndex].X;
			}
			ThreadCounts.SetNum(1);
			NumSubInvocations = 1;
		}

		// Allocate parameters buffer and fill from data providers.
		TStridedView<FComputeKernelShader::FParameters> ParameterArray = GraphBuilder.AllocParameters<FComputeKernelShader::FParameters>(KernelInvocation.ShaderParameterMetadata, NumSubInvocations);
		FComputeDataProviderRenderProxy::FDispatchData DispatchData{ KernelIndex, NumSubInvocations, bIsUnifiedDispatch, 0, 0, ParameterArray.GetStride(), reinterpret_cast<uint8*>(&ParameterArray[0]) };

		// Iterate shader parameter members to fill the dispatch data structures.
		// We assume that the members were filled out with a single data interface per member, and that the
		// order is the same one defined in the KernelInvocation.BoundProviderIndices.
		TArray<FShaderParametersMetadata::FMember> const& ParamMembers = KernelInvocation.ShaderParameterMetadata->GetMembers();
		for (int32 MemberIndex = 0; MemberIndex < ParamMembers.Num(); ++MemberIndex)
		{
			FShaderParametersMetadata::FMember const& Member = ParamMembers[MemberIndex];
			if (ensure(Member.GetBaseType() == EUniformBufferBaseType::UBMT_NESTED_STRUCT))
			{
				const int32 DataProviderIndex = KernelInvocation.BoundProviderIndices[MemberIndex];
				FComputeDataProviderRenderProxy* DataProvider = GraphInvocation.DataProviderRenderProxies[DataProviderIndex];
				if (ensure(DataProvider != nullptr))
				{
					// 1. Data interfaces sharing the same binding (primary) as the kernel should present its data in a way that
					// matches the kernel dispatch method, which can be either unified(full buffer) or non-unified (per invocation window into the full buffer)
					// 2. Data interfaces not sharing the same binding (secondary) should always provide a full view to its data (unified)
					DispatchData.bUnifiedDispatch = KernelInvocation.BoundProviderIsPrimary[MemberIndex]? bIsUnifiedDispatch : true;
					DispatchData.ParameterStructSize = Member.GetStructMetadata()->GetSize();
					DispatchData.ParameterBufferOffset = Member.GetOffset();
					DataProvider->GatherDispatchData(DispatchData);
				}
			}
		}

		// Dispatch work to the render graph.
		for (int32 SubInvocationIndex = 0; SubInvocationIndex < NumSubInvocations; ++SubInvocationIndex)
		{
			TShaderRef<FComputeKernelShader> Shader = Shaders[SubmitDesc.ShaderIndex + SubInvocationIndex];
			FIntVector GroupCount = FComputeShaderUtils::GetGroupCount(ThreadCounts[SubInvocationIndex], KernelInvocation.KernelGroupSize);

			GroupCount = FComputeShaderUtils::GetGroupCountWrapped(GroupCount.X);
			
			FComputeShaderUtils::AddPass(
				GraphBuilder,
				{},
				ERDGPassFlags::Compute | ERDGPassFlags::NeverCull,
				Shader,
				KernelInvocation.ShaderParameterMetadata,
				&ParameterArray[SubInvocationIndex],
				GroupCount
			);
		}

		// Post submit calls.
		for (int32 DataProviderIndex : KernelInvocation.PostSubmitProviderIndices)
		{
			if (FComputeDataProviderRenderProxy* DataProviderProxy = GraphInvocation.DataProviderRenderProxies[DataProviderIndex])
			{
				DataProviderProxy->PostSubmit(Context);
			}
		}

		Context.GraphKernelIndex = INDEX_NONE;

		// Enqueue readbacks.
		for (int32 DataProviderIndex : KernelInvocation.ReadbackProviderIndices)
		{
			if (FComputeDataProviderRenderProxy* DataProviderProxy = GraphInvocation.DataProviderRenderProxies[DataProviderIndex])
			{
				ReadbackDatas.Reset();
				DataProviderProxy->GetReadbackData(ReadbackDatas);

				for (const FComputeDataProviderRenderProxy::FReadbackData& ReadbackData : ReadbackDatas)
				{
					if (!ReadbackData.Buffer || ReadbackData.NumBytes == 0 || !ReadbackData.ReadbackCallback_RenderThread)
					{
						continue;
					}

					FRHIGPUBufferReadback* ReadbackRequest = new FRHIGPUBufferReadback(TEXT("ComputeFrameworkBuffer"));
					check(ReadbackRequest);

					FAsyncReadback& Readback = ActiveAsyncReadbacks.Emplace_GetRef();
					Readback.Readback = ReadbackRequest;
					Readback.NumBytes = ReadbackData.NumBytes;
					Readback.OwnerPointer = GraphInvocation.OwnerPointer;
					Readback.OnDataAvailable = *ReadbackData.ReadbackCallback_RenderThread;

					AddEnqueueCopyPass(GraphBuilder, Readback.Readback, ReadbackData.Buffer, Readback.NumBytes);
				}
			}
		}
	}

	ExternalAccessQueue.Submit(GraphBuilder);

	// Release any graph resources at the end of graph execution.
	GraphBuilder.AddPostExecuteCallback(
		[this, ExecutionGroupName=Context.ExecutionGroupName]
		{
			GraphInvocationsPerGroup.FindChecked(ExecutionGroupName).Reset();
		});
}

void FComputeGraphTaskWorker::UpdateReadbacks()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(ComputeFramework::UpdateReadbacks);

	for (int I = ActiveAsyncReadbacks.Num() - 1; I >= 0; --I)
	{
		FRHIGPUBufferReadback* Request = ActiveAsyncReadbacks[I].Readback;
		const uint32 NumBytes = ActiveAsyncReadbacks[I].NumBytes;

		if (ensure(Request) && Request->IsReady())
		{
			void* ReadbackData = Request->Lock(NumBytes);

			if (ensure(ReadbackData))
			{
				ActiveAsyncReadbacks[I].OnDataAvailable(ReadbackData, NumBytes);
			}

			ActiveAsyncReadbacks.RemoveAtSwap(I);
		}
	}
}

#if HAS_GPU_STATS
void FComputeGraphTaskWorker::FGraphInvocation::SendNullKernelTimings() const
{
	for (TPromise<FComputeFrameworkKernelTiming>& Promise : KernelTimingPromises)
	{
		Promise.SetValue(FComputeFrameworkKernelTiming{});
	}

	KernelTimingPromises.Reset();
}
#endif

FComputeGraphTaskWorker::FGraphInvocation::~FGraphInvocation()
{
	// DataProviderRenderProxy objects are created per frame and destroyed here after render work has been submitted.
	// todo[CF]: Some proxies can probably persist, but will need logic to define that and flag when they need recreating.
	for (FComputeDataProviderRenderProxy* DataProvider : DataProviderRenderProxies)
	{
		delete DataProvider;
	}
}

FComputeGraphTaskWorker::FAsyncReadback::~FAsyncReadback()
{
	if (ensure(Readback))
	{
		delete Readback;
		Readback = nullptr;
	}
}
