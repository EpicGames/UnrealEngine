// Copyright Epic Games, Inc. All Rights Reserved.

#include "ComputeFramework/ComputeKernel.h"
#include "ComputeFramework/ComputeKernelSource.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ComputeKernel)

#if WITH_EDITOR

void UComputeKernel::PostLoad()
{
	Super::PostLoad();
	if (KernelSource != nullptr)
	{
		KernelSource->ConditionalPostLoad();
	}
}

#endif // WITH_EDITOR
