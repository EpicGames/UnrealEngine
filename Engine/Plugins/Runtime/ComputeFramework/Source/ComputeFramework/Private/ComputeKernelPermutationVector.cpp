// Copyright Epic Games, Inc. All Rights Reserved.

#include "ComputeFramework/ComputeKernelPermutationVector.h"

#include "ComputeFramework/ComputeKernelPermutationSet.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ComputeKernelPermutationVector)

void FComputeKernelPermutationVector::AddPermutation(FString const& Name, uint32 NumValues)
{
	check(NumValues > 0);

	uint32& PackedPermutationBits = Permutations.FindOrAdd(Name);
	if (PackedPermutationBits != 0)
	{
		// Already found.
		FPermutationBits PermutationBits;
		PermutationBits.PackedValue = PackedPermutationBits;
		check(PermutationBits.NumValues == NumValues);
		return;
	}

	FPermutationBits PermutationBits;
	PermutationBits.BitIndex = BitCount;
	PermutationBits.NumValues = NumValues;
	PackedPermutationBits = PermutationBits.PackedValue;

	BitCount += FMath::CeilLogTwo(NumValues);
	check(BitCount <= 32);
}

void FComputeKernelPermutationVector::AddPermutationSet(struct FComputeKernelPermutationSet const& PermutationSet)
{
	for (FComputeKernelPermutationBool const& Permuation : PermutationSet.BooleanOptions)
	{
		AddPermutation(Permuation.Name, 1);
	}
}

uint32 FComputeKernelPermutationVector::GetPermutationBits(FString const& Name, uint32 PrecomputedNameHash, uint32 Value) const
{
	uint32 const* PackedPermutationBits = Permutations.FindByHash(PrecomputedNameHash, Name);
	if (!ensure(PackedPermutationBits != nullptr))
	{
		return 0;
	}

	FComputeKernelPermutationVector::FPermutationBits PermutationBits;
	PermutationBits.PackedValue = *PackedPermutationBits;
	ensure(Value < PermutationBits.NumValues);
	return (Value << PermutationBits.BitIndex);
}

uint32 FComputeKernelPermutationVector::GetPermutationBits(FString const& Name, uint32 Value) const
{
	return GetPermutationBits(Name, GetTypeHash(Name), Value);
}
