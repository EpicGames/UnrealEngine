// Copyright Epic Games, Inc. All Rights Reserved.

#include "ComputeFramework/ComputeKernelShader.h"

#include "ComputeFramework/ComputeKernelShaderType.h"

IMPLEMENT_SHADER_TYPE(, FComputeKernelShader, TEXT("/Plugin/ComputeFramework/Private/ComputeKernel.usf"), TEXT("__"), SF_Compute)

FComputeKernelShader::FComputeKernelShader(const FComputeKernelShaderType::CompiledShaderInitializerType& Initializer)
	: FShader(Initializer)
{
	const FShaderParametersMetadata& ShaderParametersMetadata = 
		static_cast<const FComputeKernelShaderType::FParameters*>(Initializer.Parameters)->ShaderParamMetadata;

	Bindings.BindForLegacyShaderParameters(
		this, 
		Initializer.PermutationId, 
		Initializer.ParameterMap, 
		ShaderParametersMetadata,
		true
		);
}
