// Copyright Epic Games, Inc. All Rights Reserved.

#include "ComputeFramework/ComputeKernelShaderCompilationManager.h"

#include "ComputeFramework/ComputeKernelShared.h"
#include "DataDrivenShaderPlatformInfo.h"
#include "HAL/FileManager.h"
#include "Misc/Paths.h"
#include "RHIShaderFormatDefinitions.inl"
#include "ShaderCompiler.h"

#if WITH_EDITOR
#include "Algo/AnyOf.h"
#include "Interfaces/IShaderFormat.h"
#include "Interfaces/ITargetPlatformManagerModule.h"
#endif

DEFINE_LOG_CATEGORY_STATIC(LogComputeKernelShaderCompiler, Log, All);

static int32 GShowComputeKernelShaderWarnings = 0;
static FAutoConsoleVariableRef CVarShowComputeKernelShaderWarnings(
	TEXT("ComputeKernel.ShowShaderCompilerWarnings"),
	GShowComputeKernelShaderWarnings,
	TEXT("When set to 1, will display all warnings from ComputeKernel shader compiles.")
	);


#if WITH_EDITOR
FComputeKernelShaderCompilationManager GComputeKernelShaderCompilationManager;

void FComputeKernelShaderCompilationManager::Tick(float DeltaSeconds)
{
	ProcessAsyncResults();
}

void FComputeKernelShaderCompilationManager::AddJobs(TArray<FShaderCommonCompileJobPtr> InNewJobs)
{
	check(IsInGameThread());
	JobQueue.Append(InNewJobs);

	for (FShaderCommonCompileJobPtr& Job : InNewJobs)
	{
		FComputeKernelShaderMapCompileResults& ShaderMapInfo = ComputeKernelShaderMapJobs.FindOrAdd(Job->Id);
		ShaderMapInfo.NumJobsQueued++;

		FShaderCompileJob* CurrentJob = Job->GetSingleShaderJob();

		CurrentJob->Input.DumpDebugInfoRootPath = GShaderCompilingManager->GetAbsoluteShaderDebugInfoDirectory() / CurrentJob->Input.ShaderPlatformName.ToString();
		FPaths::NormalizeDirectoryName(CurrentJob->Input.DumpDebugInfoRootPath);

		CurrentJob->Input.DebugExtension.Empty();
		CurrentJob->Input.DumpDebugInfoPath.Empty();
		if (GShaderCompilingManager->GetDumpShaderDebugInfo() == FShaderCompilingManager::EDumpShaderDebugInfo::Always)
		{
			CurrentJob->Input.DumpDebugInfoRootPath = GShaderCompilingManager->CreateShaderDebugInfoPath(CurrentJob->Input);
		}
	}

	GShaderCompilingManager->SubmitJobs(InNewJobs, FString(), FString());
}

void FComputeKernelShaderCompilationManager::ProcessAsyncResults()
{
	check(IsInGameThread());

	// Process the results from the shader compile worker
	for (int32 JobIndex = JobQueue.Num() - 1; JobIndex >= 0; JobIndex--)
	{
		auto CurrentJob = JobQueue[JobIndex]->GetSingleShaderJob();

		if (CurrentJob->GetStatus() != EShaderCompileJobStatus::Released)
		{
			continue;
		}

		CurrentJob->bSucceeded = CurrentJob->Output.bSucceeded;
		if (CurrentJob->Output.bSucceeded)
		{
			UE_LOGF(LogComputeKernelShaderCompiler, Verbose, "GPU shader compile succeeded. Id %d", CurrentJob->Id);
		}
		else
		{
			UE_LOGF(LogComputeKernelShaderCompiler, Verbose, "GPU shader compile failed! Id %d", CurrentJob->Id);
		}

		FComputeKernelShaderMapCompileResults& ShaderMapResults = ComputeKernelShaderMapJobs.FindChecked(CurrentJob->Id);
		ShaderMapResults.FinishedJobs.Add(JobQueue[JobIndex]);
		ShaderMapResults.bAllJobsSucceeded = ShaderMapResults.bAllJobsSucceeded && CurrentJob->bSucceeded;
		JobQueue.RemoveAt(JobIndex);
	}

	for (TMap<int32, FComputeKernelShaderMapCompileResults>::TIterator It(ComputeKernelShaderMapJobs); It; ++It)
	{
		const FComputeKernelShaderMapCompileResults& Results = It.Value();

		if (Results.FinishedJobs.Num() == Results.NumJobsQueued)
		{
			PendingFinalizeComputeKernelShaderMaps.Add(It.Key(), FComputeKernelShaderMapCompileResults(Results));
			It.RemoveCurrent();
		}
	}

	if (PendingFinalizeComputeKernelShaderMaps.Num() > 0)
	{
		ProcessCompiledComputeKernelShaderMaps(PendingFinalizeComputeKernelShaderMaps, 10);
	}
}

static bool ParseShaderCompilerError(FShaderCompilerError const& InError, bool bInCompilationSucceeded, FComputeKernelCompileMessage& OutMessage)
{
	FShaderCompilerError Error = InError;

	if (Error.StrippedErrorMessage.RemoveFromStart(TEXT("error: ")))
	{
		OutMessage.Type = FComputeKernelCompileMessage::EMessageType::Error;
	}
	else if (Error.StrippedErrorMessage.RemoveFromStart(TEXT("warning: ")))
	{
		OutMessage.Type = FComputeKernelCompileMessage::EMessageType::Warning;
	}
	else if (Error.StrippedErrorMessage.RemoveFromStart(TEXT("note: ")))
	{
		OutMessage.Type = FComputeKernelCompileMessage::EMessageType::Info;
	}
	else
	{
		// General rule for preprocessing errors - if compilation succeeded errors are warnings, otherwise errors.
		OutMessage.Type = bInCompilationSucceeded ? FComputeKernelCompileMessage::EMessageType::Warning : FComputeKernelCompileMessage::EMessageType::Error;
	}

	OutMessage.Text = Error.StrippedErrorMessage;
	OutMessage.VirtualFilePath = Error.ErrorVirtualFilePath;

	// Fix up the DataInterface generated file paths before any error reporting.
	// Magic path structure is set in ComputeGraph compilation.
	if (OutMessage.VirtualFilePath.RemoveFromStart(TEXT("/Engine/Generated/DataInterface/")))
	{
		OutMessage.VirtualFilePath.MidInline(OutMessage.VirtualFilePath.Find(TEXT("/")));
	}
	// Store any disk paths before error reporting. Can skip some known cases that won't have disk paths.
	if (OutMessage.VirtualFilePath.StartsWith(TEXT("/")) && !OutMessage.VirtualFilePath.StartsWith(TEXT("/Engine/Generated/")))
	{
		OutMessage.RealFilePath = GetShaderSourceFilePath(OutMessage.VirtualFilePath);
	}

	// Populate line and column number strings if available.
	FString Line, Column;
	// Check for "line,col" format first.
	if (!Error.ErrorLineString.Split(TEXT(","), &Line, &Column))
	{
		// Fall back to "line" which is logged by preprocessor errors.
		if (Error.ErrorLineString.IsNumeric())
		{
			Line = Error.ErrorLineString;
			Column.Empty();
		}
		else
		{
			Line.Empty();
			Column.Empty();
		}
	}

	if (Line.IsNumeric())
	{
		LexFromString(OutMessage.Line, *Line);
	}

	if (Column.IsNumeric())
	{
		LexFromString(OutMessage.ColumnStart, *Column);
		OutMessage.ColumnEnd = OutMessage.ColumnStart;

		if (Error.HasLineMarker())
		{
			for (TCHAR Character : Error.HighlightedLineMarker)
			{
				OutMessage.ColumnEnd += (Character == TEXT('~')) ? 1 : 0;
			}
		}
	}

	return true;
}

static void LogShaderCompilerErrors(FComputeKernelCompileResults const& Results)
{
	for (FComputeKernelCompileMessage const& Message : Results.Messages)
	{
		FString Path = Message.RealFilePath.IsEmpty() ? Message.VirtualFilePath : Message.RealFilePath;

		const bool bPreparePathForVisualStudioHotlink = !Message.RealFilePath.IsEmpty() && FPlatformMisc::IsDebuggerPresent();
		if (bPreparePathForVisualStudioHotlink)
		{
			// Convert path to absolute, and prepend with newline so that it is clickable in Visual Studio.
			Path = TEXT("\n") + IFileManager::Get().ConvertToAbsolutePathForExternalAppForRead(*Message.RealFilePath);
		}

		FString Line;
		if (Message.ColumnStart == Message.ColumnEnd)
		{
			Line = FString::Printf(TEXT("(%d,%d)"), Message.Line, Message.ColumnStart);
		}
		else
		{
			Line = FString::Printf(TEXT("(%d,%d-%d)"), Message.Line, Message.ColumnStart, Message.ColumnEnd);
		}

		FString MessageText = FString::Printf(TEXT("%s%s: %s"), *Path, *Line, *Message.Text);

		if (Message.Type == FComputeKernelCompileMessage::EMessageType::Warning)
		{
			UE_LOGF(LogComputeKernelShaderCompiler, Warning, "%ls", *MessageText);
		}
		else if (Message.Type == FComputeKernelCompileMessage::EMessageType::Error)
		{
			UE_LOGF(LogComputeKernelShaderCompiler, Error, "%ls", *MessageText);
		}
	}
}

void FComputeKernelShaderCompilationManager::ProcessCompiledComputeKernelShaderMaps(
	TMap<int32, FComputeKernelShaderMapFinalizeResults>& CompiledShaderMaps,
	float TimeBudget)
{
	// GetInFlightShaderMaps() is game-thread-only.
	check(IsInGameThread());

	// Keeps shader maps alive as they are passed from the shader compiler and applied to the owning kernel
	TArray<TRefCountPtr<FComputeKernelShaderMap> > LocalShaderMapReferences;
	TMap<FComputeKernelResource*, FComputeKernelShaderMap*> KernelsToUpdate;

	// Process compiled shader maps in FIFO order, in case a shader map has been enqueued multiple times,
	// Which can happen if a kernel is edited while a background compile is going on
	for (TMap<int32, FComputeKernelShaderMapFinalizeResults>::TIterator ProcessIt(CompiledShaderMaps); ProcessIt; ++ProcessIt)
	{
		TRefCountPtr<FComputeKernelShaderMap> ShaderMap = nullptr;
		TArray<FComputeKernelResource*> KernelsArray;
		bool bFoundShaderMap = false;

		for (TMap<TRefCountPtr<FComputeKernelShaderMap>, TArray<FComputeKernelResource*> >::TIterator ShaderMapIt(FComputeKernelShaderMap::GetInFlightShaderMaps()); ShaderMapIt; ++ShaderMapIt)
		{
			if (ShaderMapIt.Key()->GetCompilingId() == ProcessIt.Key())
			{
				ShaderMap = ShaderMapIt.Key();
				KernelsArray = ShaderMapIt.Value();
				bFoundShaderMap = true;
				break;
			}
		}

		if (bFoundShaderMap && ShaderMap)
		{
			FComputeKernelShaderMapFinalizeResults& CompileResults = ProcessIt.Value();
			TArray<FShaderCommonCompileJobPtr>& ResultArray = CompileResults.FinishedJobs;
			FComputeKernelCompileResults ProcessedCompileResults;
			bool bSuccess = true;

			for (int32 JobIndex = 0; JobIndex < ResultArray.Num(); JobIndex++)
			{
				FShaderCompileJob& CurrentJob = static_cast<FShaderCompileJob&>(*ResultArray[JobIndex].GetReference());
				bSuccess = bSuccess && CurrentJob.bSucceeded;

				if (bSuccess)
				{
					check(CurrentJob.Output.ShaderCode.GetShaderCodeSize() > 0);
				}

				if (GShowComputeKernelShaderWarnings || !CurrentJob.bSucceeded)
				{
					TArray<FShaderCompilerError> Errors = CurrentJob.Output.Errors;
					FShaderCompilerError::ExtractSourceLocations(Errors);

					for (int32 ErrorIndex = 0; ErrorIndex < Errors.Num(); ErrorIndex++)
					{
						FComputeKernelCompileMessage Message;
						if (ParseShaderCompilerError(Errors[ErrorIndex], CurrentJob.Output.bSucceeded, Message))
						{
							ProcessedCompileResults.Messages.AddUnique(Message);
						}
					}

					if (ProcessedCompileResults.Messages.Num())
					{
						UE_LOGF(LogComputeKernelShaderCompiler, Verbose, "There were errors for job \"%ls\"", *CurrentJob.Input.DebugGroupName)
					}
				}
				else
				{
					UE_LOGF(LogComputeKernelShaderCompiler, Verbose, "There were NO errors for job \"%ls\"", *CurrentJob.Input.DebugGroupName);
				}
			}

			bool bShaderMapComplete = true;

			if (bSuccess)
			{
				bShaderMapComplete = ShaderMap->ProcessCompilationResults(ResultArray, CompileResults.FinalizeJobIndex, TimeBudget);
			}

			if (bShaderMapComplete)
			{
				ShaderMap->SetCompiledSuccessfully(bSuccess);

				// Pass off the reference of the shader map to LocalShaderMapReferences
				LocalShaderMapReferences.Add(ShaderMap);
				FComputeKernelShaderMap::GetInFlightShaderMaps().Remove(ShaderMap);

				for (FComputeKernelResource* Kernel : KernelsArray)
				{
					FComputeKernelShaderMap* CompletedShaderMap = ShaderMap;

					Kernel->RemoveOutstandingCompileId(ShaderMap->GetCompilingId());

					// Only process results that still match the ID which requested a compile
					// This avoids applying shadermaps which are out of date and a newer one is in the async compiling pipeline
					if (Kernel->IsSame(CompletedShaderMap->GetShaderMapId()))
					{
						if (!bSuccess)
						{
							// Propagate error messages
							LogShaderCompilerErrors(ProcessedCompileResults);
							Kernel->SetCompilationResults(ProcessedCompileResults);
							KernelsToUpdate.Add(Kernel, nullptr);
						}
						else
						{
							// if we succeeded and our shader map is not complete this could be because the kernel was being edited quicker then the compile could be completed
							// Don't modify kernel for which the compiled shader map is no longer complete
							// This shouldn't happen since kernels are pretty much baked in the designated config file.
							if (CompletedShaderMap->IsComplete(Kernel, true))
							{
								KernelsToUpdate.Add(Kernel, CompletedShaderMap);
							}

							if (GShowComputeKernelShaderWarnings && ProcessedCompileResults.Messages.Num() > 0)
							{
								UE_LOGF(LogComputeKernelShaderCompiler, Warning, "Warnings while compiling ComputeKernel %ls for platform %ls:",
									*Kernel->GetFriendlyName(),
									*LegacyShaderPlatformToShaderFormat(ShaderMap->GetShaderPlatform()).ToString());

								LogShaderCompilerErrors(ProcessedCompileResults);
								Kernel->SetCompilationResults(ProcessedCompileResults);
							}
						}
					}
					else
					{
						if (CompletedShaderMap->IsComplete(Kernel, true))
						{
							const FString ShaderFormatName = FDataDrivenShaderPlatformInfo::GetShaderFormat(ShaderMap->GetShaderPlatform()).ToString();
							if (bSuccess)
							{
								const FString SuccessMessage = FString::Printf(TEXT("%s: %s shader compilation success!"), *Kernel->GetFriendlyName(), *ShaderFormatName);
								Kernel->NotifyCompilationFinished(SuccessMessage);
							}
							else
							{
								const FString FailMessage = FString::Printf(TEXT("%s: %s shader compilation failed."), *Kernel->GetFriendlyName(), *ShaderFormatName);
								Kernel->NotifyCompilationFinished(FailMessage);
							}
						}
					}
				}

				// Cleanup shader jobs and compile tracking structures
				ResultArray.Empty();
				ProcessIt.RemoveCurrent();
			}

			if (TimeBudget < 0)
			{
				break;
			}
		}
	}

	if (KernelsToUpdate.Num() > 0)
	{
		for (TMap<FComputeKernelResource*, FComputeKernelShaderMap*>::TConstIterator It(KernelsToUpdate); It; ++It)
		{
			FComputeKernelResource* Kernel = It.Key();
			FComputeKernelShaderMap* ShaderMap = It.Value();

			Kernel->SetGameThreadShaderMap(It.Value());

			ENQUEUE_RENDER_COMMAND(FSetShaderMapOnComputeKernel)(
				[Kernel, ShaderMap](FRHICommandListImmediate& RHICmdList)
				{
					Kernel->SetRenderingThreadShaderMap(ShaderMap);
				});

			if (ShaderMap && ShaderMap->CompiledSuccessfully())
			{
				const FString ShaderFormatName = FDataDrivenShaderPlatformInfo::GetShaderFormat(ShaderMap->GetShaderPlatform()).ToString();
				const FString SuccessMessage = FString::Printf(TEXT("%s: %s shader compilation success!"), 
					*Kernel->GetFriendlyName(), 
					*ShaderFormatName);
				Kernel->NotifyCompilationFinished(SuccessMessage);
			}
			else
			{
				const FString FailMessage = FString::Printf(TEXT("%s: Shader compilation failed."), *Kernel->GetFriendlyName());
				Kernel->NotifyCompilationFinished(FailMessage);
			}
		}
	}
}

void FComputeKernelShaderCompilationManager::FinishCompilation(const TCHAR* InKernelName, const TArray<int32>& ShaderMapIdsToFinishCompiling)
{
	check(!FPlatformProperties::RequiresCookedData());

	GShaderCompilingManager->FinishCompilation(NULL, ShaderMapIdsToFinishCompiling);
	ProcessAsyncResults();	// grab compiled shader maps and assign them to their resources

	check(!Algo::AnyOf(ShaderMapIdsToFinishCompiling, [this](int32 ShaderMapId) { return ComputeKernelShaderMapJobs.Contains(ShaderMapId); }));
}

void FComputeKernelShaderCompilationManager::CancelCompilation(const TCHAR* InKernelName, const TArray<int32>& ShaderMapIdsToCancelCompiling)
{
	check(!FPlatformProperties::RequiresCookedData());
	GShaderCompilingManager->CancelCompilation(NULL, ShaderMapIdsToCancelCompiling);

	for (int32 ShaderMapId : ShaderMapIdsToCancelCompiling)
	{
		JobQueue.RemoveAll([ShaderMapId](const FShaderCommonCompileJobPtr Job) { return Job->Id == ShaderMapId; });
		ComputeKernelShaderMapJobs.Remove(ShaderMapId);
	}
}

#endif // WITH_EDITOR
