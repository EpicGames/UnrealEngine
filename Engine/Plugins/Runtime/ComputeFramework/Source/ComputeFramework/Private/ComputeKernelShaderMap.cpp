// Copyright Epic Games, Inc. All Rights Reserved.

#include "ComputeFramework/ComputeKernelShader.h"

#include "ComputeFramework/ComputeKernelShared.h"
#include "ComputeFramework/ComputeKernelDerivedDataVersion.h"
#include "ComputeFramework/ComputeKernelShaderCompilationManager.h"
#include "ComputeFramework/ComputeFramework.h"
#include "DerivedDataCacheInterface.h"
#include "Interfaces/ITargetPlatformManagerModule.h"
#include "ProfilingDebugging/CookStats.h"
#include "ProfilingDebugging/DiagnosticTable.h"
#include "RHIShaderFormatDefinitions.inl"
#include "Serialization/MemoryWriter.h"
#include "Serialization/MemoryReader.h"
#include "ShaderCompiler.h"
#include "ShaderSerialization.h"
#include "Stats/StatsMisc.h"
#include "UObject/UObjectGlobals.h"
#include "DataDrivenShaderPlatformInfo.h"


#if ENABLE_COOK_STATS
namespace ComputeKernelShaderCookStats
{
	static FCookStats::FDDCResourceUsageStats UsageStats;
	static int32 ShadersCompiled = 0;
	static FCookStatsManager::FAutoRegisterCallback RegisterCookStats([](FCookStatsManager::AddStatFuncRef AddStat)
	{
		UsageStats.LogStats(AddStat, TEXT("ComputeKernelShader.Usage"), TEXT(""));
		AddStat(TEXT("ComputeKernelShader.Misc"), FCookStatsManager::CreateKeyValueArray(
			TEXT("ShadersCompiled"), ShadersCompiled
		));
	});
}
#endif


//
// Globals
//
TMap<FComputeKernelShaderMapId, FComputeKernelShaderMap*> FComputeKernelShaderMap::GIdToComputeKernelShaderMap[SP_NumPlatforms];
FCriticalSection FComputeKernelShaderMap::ShaderMapRegistryMutex;

#if WITH_EDITOR
/** 
 * Tracks FComputeKernelResource and their shader maps that are being compiled.
 * Uses a TRefCountPtr as this will be the only reference to a shader map while it is being compiled.
 */
TMap<TRefCountPtr<FComputeKernelShaderMap>, TArray<FComputeKernelResource*> > FComputeKernelShaderMap::ComputeKernelShaderMapsBeingCompiled;
#endif // WITH_EDITOR


static inline bool ShouldCacheComputeKernelShader(const FComputeKernelShaderType* InShaderType, EShaderPlatform InPlatform, const FComputeKernelResource* InKernel)
{
	return InShaderType->ShouldCache(InPlatform, InKernel) && InKernel->ShouldCache(InPlatform, InShaderType);
}


/** Hashes the kernel specific part of this shader map Id. */
uint64 FComputeKernelShaderMapId::GetComputeKernelHash() const
{
	FXxHash64Builder HashState;

	HashState.Update((const uint8*)&ShaderCodeHash, sizeof(ShaderCodeHash));
	HashState.Update((const uint8*)&FeatureLevel, sizeof(FeatureLevel));

	return HashState.Finalize().Hash;
}

/** 
* Tests this set against another for equality, disregarding override settings.
* 
* @param ReferenceSet	The set to compare against
* @return				true if the sets are equal
*/
bool FComputeKernelShaderMapId::operator==(const FComputeKernelShaderMapId& InReferenceSet) const
{
	if (  ShaderCodeHash != InReferenceSet.ShaderCodeHash
		|| FeatureLevel != InReferenceSet.FeatureLevel)
	{
		return false;
	}

	if (ShaderTypeDependencies.Num() != InReferenceSet.ShaderTypeDependencies.Num())
	{
		return false;
	}

	if (LayoutParams != InReferenceSet.LayoutParams)
	{
		return false;
	}

	for (int32 ShaderIndex = 0; ShaderIndex < ShaderTypeDependencies.Num(); ShaderIndex++)
	{
		const FShaderTypeDependency& ShaderTypeDependency = ShaderTypeDependencies[ShaderIndex];

		if (ShaderTypeDependency != InReferenceSet.ShaderTypeDependencies[ShaderIndex])
		{
			return false;
		}
	}

	return true;
}

#if WITH_EDITOR
void FComputeKernelShaderMapId::AppendKeyString(FString& OutKeyString) const
{
	OutKeyString.Appendf(TEXT("%llX"), ShaderCodeHash);
	OutKeyString.AppendChar('_');

	{
		const FSHAHash LayoutHash = GetShaderTypeLayoutHash(StaticGetTypeLayoutDesc<FComputeKernelShaderMapContent>(), LayoutParams);
		OutKeyString.AppendChar('_');
		LayoutHash.AppendString(OutKeyString);
		OutKeyString.AppendChar('_');
	}

	// Include the LayoutParams to differentiate between the frozen memory layout differences between platforms
	LayoutParams.AppendKeyString(OutKeyString);

	// Add the inputs for any shaders that are stored inline in the shader map
	AppendKeyStringShaderDependencies(MakeArrayView(ShaderTypeDependencies), LayoutParams, OutKeyString);
}

/**
 * Enqueues a compilation for a new shader of this type.
 * @param InKernel - The kernel to link the shader with.
 */
void FComputeKernelShaderType::BeginCompileShader(
	uint32 InShaderMapId,
	int32 PermutationId,
	const FComputeKernelResource* InKernel,
	FSharedShaderCompilerEnvironment* InCompilationEnvironment,
	EShaderPlatform InPlatform,
	TArray<FShaderCommonCompileJobPtr>& NewJobs,
	FShaderTarget InTarget
	)
{
	FShaderCompileJob* NewJob = GShaderCompilingManager->PrepareShaderCompileJob(
		InShaderMapId, 
		FShaderCompileJobKey(this, InPlatform, nullptr, PermutationId),
		EShaderCompileJobPriority::Normal
		);

	TCHAR const* SourceFilePath = TEXT("/Plugin/ComputeFramework/Private/ComputeKernel.usf");
	TCHAR const* GeneratedFilePath = TEXT("/Plugin/ComputeFramework/Generated/ComputeKernel.ush");

	NewJob->ShaderParameters = MakeShared<const FParameters, ESPMode::ThreadSafe>(*InKernel->GetShaderParamMetadata());
	NewJob->Input.SharedEnvironment = InCompilationEnvironment;
	NewJob->Input.Target = InTarget;
	NewJob->Input.ShaderFormat = LegacyShaderPlatformToShaderFormat(InPlatform);
	NewJob->Input.VirtualSourceFilePath = SourceFilePath;
	NewJob->Input.EntryPointName = InKernel->GetEntryPoint();
	NewJob->Input.Environment.IncludeVirtualPathToContentsMap.Add(GeneratedFilePath, InKernel->GetHLSLSource());
	UE_LOGF(LogComputeFramework, Verbose, "%ls", *InKernel->GetHLSLSource());
	
	AddUniformBufferIncludesToEnvironment(NewJob->Input.Environment, InPlatform);

	FShaderCompilerEnvironment& ShaderEnvironment = NewJob->Input.Environment;

	UE_LOGF(LogComputeFramework, Verbose, "			%ls", GetName());
	COOK_STAT(ComputeKernelShaderCookStats::ShadersCompiled++);

	InKernel->SetupCompileEnvironment(PermutationId, ShaderEnvironment);

	// Allow the shader type to modify the compile environment.
	SetupCompileEnvironment(InPlatform, InKernel, ShaderEnvironment);

	::GlobalBeginCompileShader(
		InKernel->GetFriendlyName(),
		nullptr,
		this,
		nullptr,//ShaderPipeline,
		PermutationId,
		SourceFilePath,
		*InKernel->GetEntryPoint(),
		FShaderTarget(GetFrequency(), InPlatform),
		NewJob->Input
		);

	NewJobs.Add(FShaderCommonCompileJobPtr(NewJob));
}

/**
 * Either creates a new instance of this type or returns an equivalent existing shader.
 * @param InShaderMapHash - Precomputed hash of the shader map 
 * @param InCurrentJob - Compile job that was enqueued by BeginCompileShader.
 */
FShader* FComputeKernelShaderType::FinishCompileShader(
	uint64 InShaderMapHash,
	const FShaderCompileJob& InCurrentJob,
	const FString& InDebugDescription
	) const
{
	check(InCurrentJob.bSucceeded);

	FShader* Shader = ConstructCompiled(
		FComputeKernelShaderType::CompiledShaderInitializerType(
			this,
			static_cast<const FParameters*>(InCurrentJob.ShaderParameters.Get()),
			InCurrentJob.Key.PermutationId,
			InCurrentJob.Output,
			InShaderMapHash,
			InDebugDescription
			)
		);

	InCurrentJob.Output.ParameterMap.VerifyBindingsAreComplete(GetName(), InCurrentJob.Output.Target, InCurrentJob.Key.VFType);

	return Shader;
}
#endif // WITH_EDITOR

/**
 * Finds the shader map for a kernel.
 * @param InShaderMapId - The kernel id and static parameter set identifying the shader map
 * @param InPlatform - The platform to lookup for
 * @return nullptr if no cached shader map was found.
 */
TRefCountPtr<FComputeKernelShaderMap> FComputeKernelShaderMap::FindId(const FComputeKernelShaderMapId& InShaderMapId, EShaderPlatform InPlatform)
{
	FScopeLock Lock(&ShaderMapRegistryMutex);
	TRefCountPtr<FComputeKernelShaderMap> Result = GIdToComputeKernelShaderMap[InPlatform].FindRef(InShaderMapId);
	check(Result == nullptr || !Result->bDeletedThroughDeferredCleanup);
	return Result;
}

#if WITH_EDITOR

/** Creates a string key for the derived data cache given a shader map id. */
static FString GetComputeKernelShaderMapKeyString(const FComputeKernelShaderMapId& InShaderMapId, EShaderPlatform InPlatform)
{
	const FName Format = LegacyShaderPlatformToShaderFormat(InPlatform);
	FString ShaderMapKeyString = Format.ToString() + TEXT("_") + FString(FString::FromInt(GetTargetPlatformManagerRef().ShaderFormatVersion(Format))) + TEXT("_");
	ShaderMapAppendKeyString(InPlatform, ShaderMapKeyString);
	InShaderMapId.AppendKeyString(ShaderMapKeyString);
	return FDerivedDataCacheInterface::BuildCacheKey(TEXT("CKERSM"), COMPUTEKERNEL_DERIVEDDATA_VER, *ShaderMapKeyString);
}

void FComputeKernelShaderMap::LoadFromDerivedDataCache(const FComputeKernelResource* InKernel, const FComputeKernelShaderMapId& InShaderMapId, EShaderPlatform InPlatform, TRefCountPtr<FComputeKernelShaderMap>& InOutShaderMap)
{
	if (InOutShaderMap != nullptr)
	{
		check(InOutShaderMap->GetShaderPlatform() == InPlatform);
	}
	else
	{
		// Shader map was not found in memory, try to load it from the DDC
		STAT(double ComputeKernelShaderDDCTime = 0);
		{
			SCOPE_SECONDS_COUNTER(ComputeKernelShaderDDCTime);
			COOK_STAT(auto Timer = ComputeKernelShaderCookStats::UsageStats.TimeSyncWork());

			TArray<uint8> CachedData;
			const FString DataKey = GetComputeKernelShaderMapKeyString(InShaderMapId, InPlatform);

			if (GetDerivedDataCacheRef().GetSynchronous(*DataKey, CachedData, InKernel->GetFriendlyName()))
			{
				COOK_STAT(Timer.AddHit(CachedData.Num()));
				InOutShaderMap = new FComputeKernelShaderMap();
				FMemoryReader Ar(CachedData, true);

				// Deserialize from the cached data
				InOutShaderMap->Serialize(Ar);

				checkSlow(InOutShaderMap->GetShaderMapId() == InShaderMapId);

				// Register in the global map
				InOutShaderMap->Register(InPlatform);
			}
			else
			{
				// We should be build the data later, and we can track that the resource was built there when we push it to the DDC.
				COOK_STAT(Timer.TrackCyclesOnly());
				InOutShaderMap = nullptr;
			}
		}
		INC_FLOAT_STAT_BY(STAT_ShaderCompiling_DDCLoading,(float)ComputeKernelShaderDDCTime);
	}
}

void FComputeKernelShaderMap::SaveToDerivedDataCache()
{
	COOK_STAT(auto Timer = ComputeKernelShaderCookStats::UsageStats.TimeSyncWork());

	TArray<uint8> SaveData;
	FMemoryWriter Ar(SaveData, true);
	Serialize(Ar);

	const FString DataKey = GetComputeKernelShaderMapKeyString(GetContent()->ShaderMapId, GetShaderPlatform());

	GetDerivedDataCacheRef().Put(*DataKey, SaveData, GetFriendlyName());

	COOK_STAT(Timer.AddMiss(SaveData.Num()));
}

/**
* Compiles the shaders for a kernel and caches them in this shader map.
* @param InKernel - The kernel to compile shaders for.
* @param InShaderMapId - the kernel id and set of static parameters to compile for
* @param InPlatform - The platform to compile to
*/
void FComputeKernelShaderMap::Compile(
	FComputeKernelResource* InKernel, 
	const FComputeKernelShaderMapId& InShaderMapId,
	TRefCountPtr<FSharedShaderCompilerEnvironment> InCompilationEnvironment,
	const FComputeKernelCompilationOutput& InComputeKernelCompilationOutput,
	EShaderPlatform InPlatform,
	bool bSynchronousCompile,
	bool bApplyCompletedShaderMapForRendering
	)
{
	if (FPlatformProperties::RequiresCookedData())
	{
		UE_LOGF(LogComputeFramework, Fatal, "Trying to compile ComputeKernel shader %ls at run-time, which is not supported on consoles!", *InKernel->GetFriendlyName() );
		return;
	}

	check(IsInGameThread());

	// Make sure we are operating on a referenced shader map or the below Find will cause this shader map to be deleted,
	// Since it creates a temporary ref counted pointer.
	check(NumRefs > 0);

	if (TArray<FComputeKernelResource*>* ExistingKernels = ComputeKernelShaderMapsBeingCompiled.Find(this))
	{
		check(!bSynchronousCompile);
		ExistingKernels->AddUnique(InKernel);
		return;
	}

	// Assign a unique identifier so that shaders from this shader map can be associated with it after a deferred compile
	CompilingId = FShaderCommonCompileJob::GetNextJobId();
	UE_LOGF(LogComputeFramework, Verbose, "CompilingId = %p %d", InKernel, CompilingId);
	InKernel->AddCompileId(CompilingId);

	// Setup the compilation environment.
	InKernel->SetupShaderCompilationEnvironment(InPlatform, *InCompilationEnvironment);

	// Store the kernel name for debugging purposes.
	FComputeKernelShaderMapContent* NewContent = new FComputeKernelShaderMapContent(InPlatform);
	NewContent->FriendlyName = InKernel->GetFriendlyName();
	NewContent->CompilationOutput = InComputeKernelCompilationOutput;
	NewContent->ShaderMapId = InShaderMapId;
	AssignContent(NewContent);

	uint32 NumShaders = 0;
	TArray<FShaderCommonCompileJobPtr> NewJobs;

	// Iterate over all shader types.
	for(TLinkedList<FShaderType*>::TIterator ShaderTypeIt(FShaderType::GetTypeList());ShaderTypeIt;ShaderTypeIt.Next())
	{
		FComputeKernelShaderType* ShaderType = ShaderTypeIt->GetComputeKernelShaderType();
		if (ShaderType && ShouldCacheComputeKernelShader(ShaderType, InPlatform, InKernel))
		{
			// Verify that the shader map Id contains inputs for any shaders that will be put into this shader map
			checkf(InShaderMapId.ContainsShaderType(ShaderType), TEXT("Compute shader map %s missing expected shader type %s"), *GetFriendlyName(), ShaderType->GetName());

			// Compile this ComputeKernel shader.
			TArray<FString> ShaderErrors;

			for (int32 PermutationId = 0; PermutationId < InKernel->GetNumPermutations(); ++PermutationId)
			{
				// Only compile the shader if we don't already have it
				if (!NewContent->HasShader(ShaderType, PermutationId))
				{
					ShaderType->BeginCompileShader(
						CompilingId,
						PermutationId,
						InKernel,
						InCompilationEnvironment,
						InPlatform,
						NewJobs,
						FShaderTarget(ShaderType->GetFrequency(), GetShaderPlatform())
						);
				}
				NumShaders++;
			}
		}
		else if (ShaderType)
		{
			InKernel->RemoveOutstandingCompileId(CompilingId);
			const FString ShaderFormatName = FDataDrivenShaderPlatformInfo::GetShaderFormat(InPlatform).ToString();

			FString Message = FString::Printf(TEXT("%s: Compilation not supported on %s."), 
				*InKernel->GetFriendlyName(), 
				*ShaderFormatName);
			InKernel->NotifyCompilationFinished(Message);
		}
	}

	UE_LOGF(LogComputeFramework, Verbose, "		%u Shaders", NumShaders);

	// Register this shader map in the global ComputeKernel->shadermap map
	Register(InPlatform);

	// Mark the shader map as not having been finalized with ProcessCompilationResults
	bCompilationFinalized = false;

	// Mark as not having been compiled
	bCompiledSuccessfully = false;

	if (NumShaders > 0)
	{
		GComputeKernelShaderCompilationManager.AddJobs(NewJobs);

		TArray<FComputeKernelResource*> NewCorrespondingKernels;
		NewCorrespondingKernels.Add(InKernel);
		ComputeKernelShaderMapsBeingCompiled.Add(this, NewCorrespondingKernels);
	}

	// Compile the shaders for this shader map now if not deferring and deferred compiles are not enabled globally
	if (bSynchronousCompile)
	{
		TArray<int32> CurrentShaderMapId;
		CurrentShaderMapId.Add(CompilingId);
		GComputeKernelShaderCompilationManager.FinishCompilation(*NewContent->FriendlyName, CurrentShaderMapId);
	}
}

FShader* FComputeKernelShaderMap::ProcessCompilationResultsForSingleJob(FShaderCompileJob& CurrentJob, uint64 InShaderMapHash)
{
	check(CurrentJob.Id == CompilingId);

	GetResourceCode()->AddShaderCompilerOutput(CurrentJob.Output, CurrentJob.Key, CurrentJob.Input.GenerateDebugInfo());

	FShader* Shader = nullptr;

	const FComputeKernelShaderType* ComputeKernelShaderType = CurrentJob.Key.ShaderType->GetComputeKernelShaderType();
	check(ComputeKernelShaderType);
	Shader = ComputeKernelShaderType->FinishCompileShader(InShaderMapHash, CurrentJob, GetContent()->FriendlyName);
	bCompiledSuccessfully = CurrentJob.bSucceeded;

	FComputeKernelShader* ComputeKernelShader = static_cast<FComputeKernelShader*>(Shader);
	check(Shader);
	check(!GetContent()->HasShader(ComputeKernelShaderType, CurrentJob.Key.PermutationId));
	return GetMutableContent()->FindOrAddShader(ComputeKernelShaderType->GetHashedName(), CurrentJob.Key.PermutationId, Shader);
}

bool FComputeKernelShaderMap::ProcessCompilationResults(const TArray<FShaderCommonCompileJobPtr>& InCompilationResults, int32& InOutJobIndex, float& InOutTimeBudget)
{
	check(InOutJobIndex < InCompilationResults.Num());

	double StartTime = FPlatformTime::Seconds();

	uint64 ShaderMapHash = GetContent()->ShaderMapId.GetComputeKernelHash();

	do
	{
		ProcessCompilationResultsForSingleJob(static_cast<FShaderCompileJob&>(*InCompilationResults[InOutJobIndex].GetReference()), ShaderMapHash);

		InOutJobIndex++;
		
		double NewStartTime = FPlatformTime::Seconds();
		InOutTimeBudget -= NewStartTime - StartTime;
		StartTime = NewStartTime;
	}
	while ((InOutTimeBudget > 0.0f) && (InOutJobIndex < InCompilationResults.Num()));

	if (InOutJobIndex == InCompilationResults.Num())
	{
		FinalizeContent();

		SaveToDerivedDataCache();
		// The shader map can now be used on the rendering thread
		bCompilationFinalized = true;
		return true;
	}

	return false;
}

bool FComputeKernelShaderMap::TryToAddToExistingCompilationTask(FComputeKernelResource* InKernel)
{
	check(NumRefs > 0);
	check(IsInGameThread());

	if (TArray<FComputeKernelResource*>* CorrespondingKernels = ComputeKernelShaderMapsBeingCompiled.Find(this))
	{
		CorrespondingKernels->AddUnique(InKernel);

		UE_LOGF(LogComputeFramework, Verbose, "TryToAddToExistingCompilationTask %p %d", InKernel, GetCompilingId());

		return true;
	}

	return false;
}
#endif // WITH_EDITOR

bool FComputeKernelShaderMap::IsComputeKernelShaderComplete(const FComputeKernelResource* InKernel, const FComputeKernelShaderType* InShaderType, bool bSilent)
{
	// If we should cache this kernel, it's incomplete if the shader is missing
	if (ShouldCacheComputeKernelShader(InShaderType, GetShaderPlatform(), InKernel))
	{
		for (int32 PermutationId = 0; PermutationId < InKernel->GetNumPermutations(); ++PermutationId)
		{
			if (!GetContent()->HasShader((FShaderType*)InShaderType, PermutationId))
			{
				if (!bSilent)
				{
					UE_LOGF(LogComputeFramework, Warning, "Incomplete shader %ls, missing FComputeKernelShader %ls.", *InKernel->GetFriendlyName(), InShaderType->GetName());
				}
				return false;
			}
		}
	}

	return true;
}

bool FComputeKernelShaderMap::IsComplete(const FComputeKernelResource* InKernel, bool bSilent)
{
	// Make sure we are operating on a referenced shader map or the below Find will cause this shader map to be deleted,
	// Since it creates a temporary ref counted pointer.
	check(NumRefs > 0);

#if WITH_EDITOR
	check(IsInGameThread());
	if (ComputeKernelShaderMapsBeingCompiled.Contains(this))
	{
		check(!bCompilationFinalized);
		return false;
	}
#endif

	// Iterate over all shader types.
	for(TLinkedList<FShaderType*>::TIterator ShaderTypeIt(FShaderType::GetTypeList());ShaderTypeIt;ShaderTypeIt.Next())
	{
		// Find this shader type in the kernel's shader map.
		const FComputeKernelShaderType* ShaderType = ShaderTypeIt->GetComputeKernelShaderType();
		if (ShaderType && !IsComputeKernelShaderComplete(InKernel, ShaderType, bSilent))
		{
			return false;
		}
	}

	return true;
}

void FComputeKernelShaderMap::GetShaderList(TMap<FShaderId, TShaderRef<FShader>>& OutShaders) const
{
	GetContent()->GetShaderList(*this, 0u, OutShaders);
}

void FComputeKernelShaderMap::GetShaderList(TMap<FHashedName, TShaderRef<FShader>>& OutShaders) const
{
	GetContent()->GetShaderList(*this, OutShaders);
}

void FComputeKernelShaderMap::GetShaderPipelineList(TArray<FShaderPipelineRef>& OutShaderPipelines) const
{
	GetContent()->GetShaderPipelineList(*this, OutShaderPipelines, FShaderPipeline::EAll);
}

/**
 * Registers a ComputeKernel shader map in the global map.
 */
void FComputeKernelShaderMap::Register(EShaderPlatform InShaderPlatform)
{
	FScopeLock Lock(&ShaderMapRegistryMutex);
	if (!bRegistered)
	{
		INC_DWORD_STAT(STAT_Shaders_NumShaderMaps);
		bRegistered = true;
	}
	FComputeKernelShaderMap* Existing = GIdToComputeKernelShaderMap[GetShaderPlatform()].FindRef(GetContent()->ShaderMapId);
	checkf(Existing == nullptr || Existing == this, TEXT("ComputeKernel shader map collision for id %s"), *GetContent()->FriendlyName);

	GIdToComputeKernelShaderMap[GetShaderPlatform()].Add(GetContent()->ShaderMapId, this);
}

void FComputeKernelShaderMap::AddRef()
{
	FScopeLock Lock(&ShaderMapRegistryMutex);
	check(!bDeletedThroughDeferredCleanup);
	FPlatformAtomics::InterlockedIncrement(&NumRefs);
}

void FComputeKernelShaderMap::Release()
{
	bool bShouldBeginCleanup = false;
	{
		FScopeLock Lock(&ShaderMapRegistryMutex);
		check(NumRefs > 0);
		if (FPlatformAtomics::InterlockedDecrement(&NumRefs) == 0)
		{
			if (bRegistered)
			{
				DEC_DWORD_STAT(STAT_Shaders_NumShaderMaps);
				GIdToComputeKernelShaderMap[GetShaderPlatform()].Remove(GetContent()->ShaderMapId);
				bRegistered = false;
			}

			check(!bDeletedThroughDeferredCleanup);
			bDeletedThroughDeferredCleanup = true;
			bShouldBeginCleanup = true;
		}
	}

	if (bShouldBeginCleanup)
	{
		// BeginCleanup can run arbitrary deferred-cleanup work, so invoke it outside the lock.
		BeginCleanup(this);
	}
}

FComputeKernelShaderMap::FComputeKernelShaderMap() :
	CompilingId(1),
	NumRefs(0),
	bDeletedThroughDeferredCleanup(false),
	bRegistered(false),
	bCompilationFinalized(true),
	bCompiledSuccessfully(true),
	bIsPersistent(true) 
{
	checkSlow(IsInGameThread() || IsAsyncLoading());
}

FComputeKernelShaderMap::~FComputeKernelShaderMap()
{ 
	checkSlow(IsInGameThread() || IsAsyncLoading());
	check(bDeletedThroughDeferredCleanup);
	check(!bRegistered);
}

bool FComputeKernelShaderMap::Serialize(FArchive& Ar)
{
	// Note: This is saved to the DDC, not into packages (except when cooked)
	// Backwards compatibility therefore will not work based on the version of Ar
	// Instead, just bump COMPUTEKERNEL_DERIVEDDATA_VER
	FShaderSerializeContext Ctx(Ar);
	return Super::Serialize(Ctx);
}

#if WITH_EDITOR

void FComputeKernelShaderMap::RemovePending(FComputeKernelResource* InKernel)
{
	check(IsInGameThread());

	TArray<uint32> RemovedCompilingIds;
	for (TMap<TRefCountPtr<FComputeKernelShaderMap>, TArray<FComputeKernelResource*> >::TIterator It(ComputeKernelShaderMapsBeingCompiled); It; ++It)
	{
		TArray<FComputeKernelResource*>& Kernels = It.Value();
		int32 Result = Kernels.Remove(InKernel);
		if (Result)
		{
			RemovedCompilingIds.Add(It.Key()->CompilingId);
		}
	}

	for (uint32 CompilingIdToRemove : RemovedCompilingIds)
	{
		InKernel->RemoveOutstandingCompileId(CompilingIdToRemove);
		InKernel->NotifyCompilationFinished(FString::Printf(TEXT("%s: Removing compilation."), *InKernel->GetFriendlyName()));
	}
}

#endif // WITH_EDITOR