// Copyright Epic Games, Inc. All Rights Reserved.

#include "ComputeFramework/ComputeKernelShared.h"

#include "ComputeFramework/ComputeKernelPermutationSet.h"
#include "ComputeFramework/ComputeKernelPermutationVector.h"
#include "ComputeFramework/ComputeKernelShaderType.h"
#include "ComputeFramework/ComputeKernelShader.h"
#include "ComputeFramework/ComputeKernelShaderCompilationManager.h"
#include "ComputeFramework/ComputeKernelSource.h"
#include "ComputeFramework/ComputeFramework.h"
#include "ComputeFramework/ShaderParameterMetadataAllocation.h"
#include "Interfaces/ITargetPlatform.h"
#include "RendererInterface.h"
#include "DataDrivenShaderPlatformInfo.h"
#include "RHIShaderFormatDefinitions.inl"
#include "ShaderCompiler.h"
#include "ShaderParameterMetadataBuilder.h"
#include "Stats/StatsMisc.h"
#include "TextureResource.h"
#include "UObject/CoreObjectVersion.h"
#include "UObject/UObjectHash.h"
#include "UObject/UObjectIterator.h"

IMPLEMENT_TYPE_LAYOUT(FComputeKernelCompilationOutput);
IMPLEMENT_TYPE_LAYOUT(FComputeKernelShaderMapId);
IMPLEMENT_TYPE_LAYOUT(FComputeKernelShaderMapContent);

FComputeKernelResource::FComputeKernelResource()
	: bContainsInlineShaders(false)
	, bLoadedCookedShaderMapId(false)
{
}

FComputeKernelResource::~FComputeKernelResource()
{
#if WITH_EDITOR
	FComputeKernelShaderMap::RemovePending(this);
#endif
}

#if WITH_EDITOR
/** Populates OutEnvironment with defines needed to compile shaders for this kernel. */
void FComputeKernelResource::SetupShaderCompilationEnvironment(EShaderPlatform InPlatform, FShaderCompilerEnvironment& OutEnvironment) const
{
	for (FComputeKernelDefinition const& Define : ShaderDefinitionSet->Defines)
	{
		OutEnvironment.SetDefine(*Define.Symbol, *Define.Define);
	}

	OutEnvironment.IncludeVirtualPathToContentsMap.Append(AdditionalSources);
	OutEnvironment.IncludeVirtualPathToContentsMap.Append(GeneratedSources);
}
#endif // WITH_EDITOR

bool FComputeKernelResource::ShouldCache(EShaderPlatform InPlatform, const FShaderType* InShaderType) const
{
	check(InShaderType->GetComputeKernelShaderType())
	return ComputeFramework::IsSupported(InPlatform);
}

void FComputeKernelResource::NotifyCompilationFinished(FString const& ResultMessage)
{
	UE_LOGF(LogComputeFramework, Log, "%ls", *ResultMessage);
	FComputeKernelCompileMessage Message;
	Message.Type = FComputeKernelCompileMessage::EMessageType::Info;
	Message.Text = ResultMessage;
	CompilationResults.Messages.Add(Message);
	OnCompilationCompleteDelegate.ExecuteIfBound(this);
}

#if WITH_EDITOR
void FComputeKernelResource::CancelCompilation()
{
	if (IsInGameThread())
	{
		TArray<int32> ShaderMapIdsToFinish;
		GetShaderMapIDsWithUnfinishedCompilation(ShaderMapIdsToFinish);
		GComputeKernelShaderCompilationManager.CancelCompilation(*GetFriendlyName(), ShaderMapIdsToFinish);

		FComputeKernelShaderMap::RemovePending(this);
		OutstandingCompileShaderMapIds.Empty();

		if (!ShaderMapIdsToFinish.IsEmpty())
		{
			FString Message = FString::Printf(TEXT("%s: Compilation cancelled."), *GetFriendlyName());
			NotifyCompilationFinished(Message);
		}
	}
}
#endif // WITH_EDITOR

void FComputeKernelResource::RemoveOutstandingCompileId(const int32 InOldOutstandingCompileShaderMapId)
{
	if (0 <= OutstandingCompileShaderMapIds.Remove(InOldOutstandingCompileShaderMapId))
	{
		UE_LOGF(LogComputeFramework, Verbose, "RemoveOutstandingCompileId %p %d", this, InOldOutstandingCompileShaderMapId);
	}
}

void FComputeKernelResource::Invalidate()
{
#if WITH_EDITOR
	CancelCompilation();
#endif
	ReleaseShaderMap();
}

bool FComputeKernelResource::IsSame(const FComputeKernelShaderMapId& InIdentifier) const
{
	return
		InIdentifier.ShaderCodeHash == ShaderCodeHash &&
		InIdentifier.FeatureLevel == FeatureLevel;
}

void FComputeKernelResource::GetDependentShaderTypes(EShaderPlatform InPlatform, TArray<FShaderType*>& OutShaderTypes) const
{
	for (TLinkedList<FShaderType*>::TIterator ShaderTypeIt(FShaderType::GetTypeList()); ShaderTypeIt; ShaderTypeIt.Next())
	{
		FComputeKernelShaderType* ShaderType = ShaderTypeIt->GetComputeKernelShaderType();

		if ( ShaderType && ShaderType->ShouldCache(InPlatform, this) && ShouldCache(InPlatform, ShaderType) )
		{
			OutShaderTypes.Add(ShaderType);
		}
	}

	OutShaderTypes.Sort(FCompareShaderTypes());
}

void FComputeKernelResource::GetShaderMapId(EShaderPlatform InPlatform, const ITargetPlatform* TargetPlatform, FComputeKernelShaderMapId& OutId) const
{
	if (bLoadedCookedShaderMapId)
	{
		OutId = CookedShaderMapId;
	}
	else
	{
		TArray<FShaderType*> ShaderTypes;
		GetDependentShaderTypes(InPlatform, ShaderTypes);

		OutId.FeatureLevel = GetFeatureLevel();
		OutId.ShaderCodeHash = ShaderCodeHash;
#if WITH_EDITOR
		OutId.SetShaderDependencies(ShaderTypes, InPlatform);
		if (TargetPlatform)
		{
			OutId.LayoutParams.InitializeForPlatform(TargetPlatform->IniPlatformName(), TargetPlatform->HasEditorOnlyData());
		}
		else
		{
			OutId.LayoutParams.InitializeForCurrent();
		}
#else
		if (TargetPlatform != nullptr)
		{
			UE_LOGF(LogComputeFramework, Error, "FComputeKernelResource::GetShaderMapId: TargetPlatform is not null, but a cooked executable cannot target platforms other than its own.");
		}
		OutId.LayoutParams.InitializeForCurrent();
#endif
	}
}

void FComputeKernelResource::ReleaseShaderMap()
{
	if (GameThreadShaderMap)
	{
		GameThreadShaderMap = nullptr;

		FComputeKernelResource* Kernel = this;
		ENQUEUE_RENDER_COMMAND(ReleaseShaderMap)(
			[Kernel](FRHICommandListImmediate& RHICmdList)
			{
				Kernel->SetRenderingThreadShaderMap(nullptr);
			});
	}
}

void FComputeKernelResource::DiscardShaderMap()
{
	check(RenderingThreadShaderMap == nullptr);
	GameThreadShaderMap = nullptr;
}

void FComputeKernelResource::SerializeShaderMap(FArchive& Ar)
{
	bool bCooked = Ar.IsCooking();
	Ar << bCooked;

	if (FPlatformProperties::RequiresCookedData() && !bCooked && Ar.IsLoading())
	{
		UE_LOGF(LogComputeFramework, Fatal, "This platform requires cooked packages, and shaders were not cooked into this kernel %ls.", *GetFriendlyName());
	}

	if (bCooked)
	{
		if (Ar.IsCooking())
		{
#if WITH_EDITOR
			FinishCompilation();

			bool bValid = GameThreadShaderMap != nullptr && GameThreadShaderMap->CompiledSuccessfully();
			Ar << bValid;

			if (bValid)
			{
				GameThreadShaderMap->AssociateWithAsset(AssetPath);

				GameThreadShaderMap->Serialize(Ar);
			}
#endif
		}
		else
		{
			bool bValid = false;
			Ar << bValid;

			if (bValid)
			{
				TRefCountPtr<FComputeKernelShaderMap> LoadedShaderMap = new FComputeKernelShaderMap();
				bool bSuccessfullyLoaded = LoadedShaderMap->Serialize(Ar);

				// Toss the loaded shader data if this is a server only instance or if it's for a different RHI than the current one.
				// todo[CF] Don't cook it in the first place
				if (bSuccessfullyLoaded && FApp::CanEverRender())
				{
					GameThreadShaderMap = RenderingThreadShaderMap = LoadedShaderMap;
					GameThreadShaderMap->GetResource()->SetOwnerName(GetOwnerFName());
				}
			}
		}
	}
}

void FComputeKernelResource::SetupResource(
	ERHIFeatureLevel::Type InFeatureLevel,
	FString const& InFriendlyName,
	FString const& InShaderEntryPoint,
	FString const& InShaderHashKey,
	FString const& InShaderSource,
	TMap<FString, FString> const& InAdditionalSources,
	TMap<FString, FString> const& InGeneratedSources,
	TSharedPtr<FComputeKernelDefinitionSet>& InShaderDefinitionSet,
	TSharedPtr<FComputeKernelPermutationVector>& InShaderPermutationVector,
	TUniquePtr<FShaderParametersMetadataAllocations>& InShaderParameterMetadataAllocations,
	FShaderParametersMetadata* InShaderParameterMetadata,
	FName const& InAssetPath
	)
{
	FeatureLevel = InFeatureLevel;
	FriendlyName = InFriendlyName;
	ShaderEntryPoint = InShaderEntryPoint;
	ShaderCodeHash = GetTypeHash(InShaderHashKey);
	ShaderSource = InShaderSource;
	AdditionalSources = InAdditionalSources;
	GeneratedSources = InGeneratedSources;
	ShaderDefinitionSet = InShaderDefinitionSet;
	ShaderPermutationVector = InShaderPermutationVector;
	ShaderParameterMetadataAllocations = MoveTemp(InShaderParameterMetadataAllocations);
	ShaderParameterMetadata = InShaderParameterMetadata;
	CompilationResults.Messages.Reset();
	AssetPath = InAssetPath;
}

int32 FComputeKernelResource::GetNumPermutations() const 
{
	// todo[CF]: ShaderPermutationVector is not serialized/restored in non-editor builds!
	// todo[CF]: Support sparse permutation bits.
	return ShaderPermutationVector.IsValid() ? 1 << ShaderPermutationVector->BitCount : 1;
}

#if WITH_EDITOR
void FComputeKernelResource::SetupCompileEnvironment(int32 InPermutationId, FShaderCompilerEnvironment& OutShaderEnvironment) const
{
 	for (TPair<FString, uint32> const& Permutation : ShaderPermutationVector->Permutations)
	{
		FComputeKernelPermutationVector::FPermutationBits PermutationBits;
		PermutationBits.PackedValue = Permutation.Value;
		const uint32 Mask = (1 << FMath::CeilLogTwo(PermutationBits.NumValues)) - 1;
		const uint32 Value = (InPermutationId >> PermutationBits.BitIndex) & Mask;
		if (Value > 0 && Value < PermutationBits.NumValues)
		{
			OutShaderEnvironment.SetDefine(*Permutation.Key, Value);
		}
	}
}
#endif // WITH_EDITOR

void FComputeKernelResource::SetRenderingThreadShaderMap(FComputeKernelShaderMap* InShaderMap)
{
	check(IsInRenderingThread());
	RenderingThreadShaderMap = InShaderMap;
}

bool FComputeKernelResource::IsCompilationFinished() const
{
	bool bRet = GameThreadShaderMap && GameThreadShaderMap.IsValid() && GameThreadShaderMap->IsCompilationFinalized();
	if (OutstandingCompileShaderMapIds.Num() == 0)
	{
		return true;
	}
	return bRet;
}

bool FComputeKernelResource::CacheShaders(EShaderPlatform InPlatform, const ITargetPlatform* TargetPlatform, bool bApplyCompletedShaderMapForRendering, bool bSynchronous)
{
	FComputeKernelShaderMapId ShaderMapId;
	GetShaderMapId(InPlatform, TargetPlatform, ShaderMapId);
	return CacheShaders(ShaderMapId, InPlatform, bApplyCompletedShaderMapForRendering, bSynchronous);
}

bool FComputeKernelResource::CacheShaders(const FComputeKernelShaderMapId& InShaderMapId, EShaderPlatform InPlatform, bool bApplyCompletedShaderMapForRendering, bool bSynchronous)
{
	bool bSucceeded = false;
	{
		// If we loaded this material with inline shaders, use what was loaded (GameThreadShaderMap) instead of looking in the DDC
		if (bContainsInlineShaders)
		{
			TRefCountPtr<FComputeKernelShaderMap> ExistingShaderMap;

			if (GameThreadShaderMap)
			{
				// Note: in the case of an inlined shader map, the shadermap Id will not be valid because we stripped some editor-only data needed to create it
				// Get the shadermap Id from the shadermap that was inlined into the package, if it exists
				ExistingShaderMap = FComputeKernelShaderMap::FindId(GameThreadShaderMap->GetShaderMapId(), InPlatform);
			}

			// Re-use an identical shader map in memory if possible, removing the reference to the inlined shader map
			if (ExistingShaderMap)
			{
				GameThreadShaderMap = ExistingShaderMap;
			}
			else if (GameThreadShaderMap)
			{
				// We are going to use the inlined shader map, register it so it can be re-used by other materials
				GameThreadShaderMap->Register(InPlatform);
			}
		}
		else
		{
			// Find the kernel's cached shader map.
			GameThreadShaderMap = FComputeKernelShaderMap::FindId(InShaderMapId, InPlatform);

			// Attempt to load from the derived data cache if we are uncooked
			if ((!GameThreadShaderMap || !GameThreadShaderMap->IsComplete(this, true)) && !FPlatformProperties::RequiresCookedData())
			{
#if WITH_EDITOR
				FComputeKernelShaderMap::LoadFromDerivedDataCache(this, InShaderMapId, InPlatform, GameThreadShaderMap);
				if (GameThreadShaderMap == nullptr || !GameThreadShaderMap->IsValid())
				{
					UE_LOGF(LogComputeFramework, Log, "Loading shader for kernel %ls from DDC failed. Shader needs recompile.", *GetFriendlyName());
				}
#endif // WITH_EDITOR
			}
		}
	}

	bool bAssumeShaderMapIsComplete = false;
#if UE_BUILD_SHIPPING || UE_BUILD_TEST
	bAssumeShaderMapIsComplete = (bContainsInlineShaders || FPlatformProperties::RequiresCookedData());
#endif

#if WITH_EDITOR
	if (GameThreadShaderMap && GameThreadShaderMap->TryToAddToExistingCompilationTask(this))
	{
		OutstandingCompileShaderMapIds.AddUnique(GameThreadShaderMap->GetCompilingId());
		UE_LOGF(LogComputeFramework, Verbose, "CacheShaders AddUniqueExisting %p %d", this, GameThreadShaderMap->GetCompilingId());

		GameThreadShaderMap = nullptr;
		bSucceeded = true;
	}
	else
#endif // WITH_EDITOR
	if (!GameThreadShaderMap || !(bAssumeShaderMapIsComplete || GameThreadShaderMap->IsComplete(this, false)))
	{
		if (bContainsInlineShaders || FPlatformProperties::RequiresCookedData())
		{
			UE_LOGF(LogComputeFramework, Log, "Can't compile %ls with cooked content!", *GetFriendlyName());
			GameThreadShaderMap = nullptr;
		}
		else
		{
#if WITH_EDITOR
			UE_LOGF(LogComputeFramework, Log, "%ls cached shader map for kernel %ls, compiling.", GameThreadShaderMap ? TEXT("Incomplete") : TEXT("Missing"), *GetFriendlyName());

			// If there's no cached shader map for this kernel compile a new one.
			// This is just kicking off the compile, GameThreadShaderMap will not be complete yet
			bSucceeded = BeginCompileShaderMap(InShaderMapId, InPlatform, GameThreadShaderMap, bApplyCompletedShaderMapForRendering, bSynchronous);
#endif

			if (!bSucceeded)
			{
				GameThreadShaderMap = nullptr;
			}
		}
	}
	else
	{
		const FString ShaderFormatName = FDataDrivenShaderPlatformInfo::GetShaderFormat(InPlatform).ToString();
		NotifyCompilationFinished(FString::Printf(TEXT("%s: Loaded %s shaders from DDC."), *GetFriendlyName(), *ShaderFormatName));
		bSucceeded = true;
	}

	if (bApplyCompletedShaderMapForRendering)
	{
		FComputeKernelResource* Kernel = this;
		FComputeKernelShaderMap* LoadedShaderMap = GameThreadShaderMap;
		ENQUEUE_RENDER_COMMAND(FSetShaderMapOnComputeKernel)(
			[Kernel, LoadedShaderMap](FRHICommandListImmediate& RHICmdList)
			{
				Kernel->SetRenderingThreadShaderMap(LoadedShaderMap);
			});
	}

	return bSucceeded;
}

bool FComputeKernelResource::CacheInlineShaders(EShaderPlatform InPlatform)
{
	return bContainsInlineShaders && CacheShaders(InPlatform, nullptr, true, true);
}

#if WITH_EDITOR
void FComputeKernelResource::FinishCompilation()
{
	TArray<int32> ShaderMapIdsToFinish;
	GetShaderMapIDsWithUnfinishedCompilation(ShaderMapIdsToFinish);

	if (ShaderMapIdsToFinish.Num() > 0)
	{
		for (int32 i = 0; i < ShaderMapIdsToFinish.Num(); i++)
		{
			UE_LOGF(LogComputeFramework, Verbose, "FinishCompilation()[%d] %ls id %d!", i, *GetFriendlyName(), ShaderMapIdsToFinish[i]);
		}
	
		// Block until the shader maps that we will save have finished being compiled
		GComputeKernelShaderCompilationManager.FinishCompilation(*GetFriendlyName(), ShaderMapIdsToFinish);

		// Shouldn't have anything left to do...
		TArray<int32> ShaderMapIdsToFinish2;
		GetShaderMapIDsWithUnfinishedCompilation(ShaderMapIdsToFinish2);
		ensure(ShaderMapIdsToFinish2.Num() == 0);
	}
}
#endif // WITH_EDITOR

TShaderRef<FComputeKernelShader> FComputeKernelResource::GetShader(int32 PermutationId) const
{
	check(!GIsThreadedRendering || !IsInGameThread());
	if (!GIsEditor || RenderingThreadShaderMap)
	{
		return RenderingThreadShaderMap->GetShader<FComputeKernelShader>(PermutationId);
	}
	return TShaderRef<FComputeKernelShader>();
}

TShaderRef<FComputeKernelShader> FComputeKernelResource::GetShaderGameThread(int32 PermutationId) const
{
	if (GameThreadShaderMap)
	{
		return GameThreadShaderMap->GetShader<FComputeKernelShader>(PermutationId);
	}
	return TShaderRef<FComputeKernelShader>();
}

#if WITH_EDITOR
void FComputeKernelResource::GetShaderMapIDsWithUnfinishedCompilation(TArray<int32>& OutShaderMapIds)
{
	// Build an array of the shader map Id's are not finished compiling.
	if (GameThreadShaderMap && GameThreadShaderMap.IsValid() && !GameThreadShaderMap->IsCompilationFinalized())
	{
		OutShaderMapIds.Add(GameThreadShaderMap->GetCompilingId());
	}
	else if (OutstandingCompileShaderMapIds.Num() != 0)
	{
		OutShaderMapIds.Append(OutstandingCompileShaderMapIds);
	}
}

/**
 * Compiles this kernel for Platform, storing the result in OutShaderMap
 *
 * @param InShaderMapId - the set of static parameters to compile
 * @param InPlatform - the platform to compile for
 * @param OutShaderMap - the shader map to compile
 * @return - true if compile succeeded or was not necessary (shader map for ShaderMapId was found and was complete)
 */
bool FComputeKernelResource::BeginCompileShaderMap(const FComputeKernelShaderMapId& InShaderMapId, EShaderPlatform InPlatform, TRefCountPtr<FComputeKernelShaderMap>& OutShaderMap, bool bApplyCompletedShaderMapForRendering, bool bSynchronous)
{
	bool bSuccess = false;

	STAT(double ComputeKernelCompileTime = 0);

	SCOPE_SECONDS_COUNTER(ComputeKernelCompileTime);

	TRefCountPtr<FComputeKernelShaderMap> NewShaderMap = new FComputeKernelShaderMap();

	// Create a shader compiler environment for the material that will be shared by all jobs from this material
	TRefCountPtr<FSharedShaderCompilerEnvironment> Environment = new FSharedShaderCompilerEnvironment();

	// Compile the shaders for the kernel.
	FComputeKernelCompilationOutput CompilationOutput;
	NewShaderMap->Compile(this, InShaderMapId, Environment, CompilationOutput, InPlatform, bSynchronous, bApplyCompletedShaderMapForRendering);

	if (bSynchronous)
	{
		// If this is a synchronous compile, assign the compile result to the output
		OutShaderMap = NewShaderMap->CompiledSuccessfully() ? NewShaderMap : nullptr;
	}
	else
	{
		UE_LOGF(LogComputeFramework, Verbose, "BeginCompileShaderMap AddUnique %p %d", this, NewShaderMap->GetCompilingId());
		OutstandingCompileShaderMapIds.AddUnique(NewShaderMap->GetCompilingId());
		
		// Async compile, use nullptr to detect it if used
		OutShaderMap = nullptr;
	}

	return true;
}

void FComputeKernelShaderMapId::SetShaderDependencies(const TArray<FShaderType*>& InShaderTypes, EShaderPlatform InShaderPlatform)
{
	if (!FPlatformProperties::RequiresCookedData())
	{
		for (FShaderType* ShaderType : InShaderTypes)
		{
			if (ShaderType != nullptr)
			{
				FShaderTypeDependency Dependency;
				Dependency.ShaderTypeName = ShaderType->GetHashedName();
				Dependency.SourceHash = ShaderType->GetSourceHash(InShaderPlatform);
				ShaderTypeDependencies.Add(Dependency);
			}
		}
	}
}

#endif // WITH_EDITOR

bool FComputeKernelShaderMapId::ContainsShaderType(const FShaderType* ShaderType) const
{
	for (int32 TypeIndex = 0; TypeIndex < ShaderTypeDependencies.Num(); TypeIndex++)
	{
		if (ShaderTypeDependencies[TypeIndex].ShaderTypeName == ShaderType->GetHashedName())
		{
			return true;
		}
	}

	return false;
}
