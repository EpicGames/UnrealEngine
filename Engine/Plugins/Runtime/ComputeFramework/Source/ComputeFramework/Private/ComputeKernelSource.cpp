// Copyright Epic Games, Inc. All Rights Reserved.

#include "ComputeFramework/ComputeKernelSource.h"

#if WITH_EDITOR

#include "Internationalization/Regex.h"

/** Strip regex escape characters that can be in EntryPoint since it is a user editable string. */
static FString StripRegexCharacters(const FString& Input)
{
	FString Result = Input;
	Result.ReplaceInline(TEXT("\\"), TEXT(""));  // must be first                                                                                                                   
	Result.ReplaceInline(TEXT("."), TEXT(""));
	Result.ReplaceInline(TEXT("*"), TEXT(""));
	Result.ReplaceInline(TEXT("+"), TEXT(""));
	Result.ReplaceInline(TEXT("?"), TEXT(""));
	Result.ReplaceInline(TEXT("^"), TEXT(""));
	Result.ReplaceInline(TEXT("$"), TEXT(""));
	Result.ReplaceInline(TEXT("{"), TEXT(""));
	Result.ReplaceInline(TEXT("}"), TEXT(""));
	Result.ReplaceInline(TEXT("["), TEXT(""));
	Result.ReplaceInline(TEXT("]"), TEXT(""));
	Result.ReplaceInline(TEXT("("), TEXT(""));
	Result.ReplaceInline(TEXT(")"), TEXT(""));
	Result.ReplaceInline(TEXT("|"), TEXT(""));
	Result.ReplaceInline(TEXT("/"), TEXT(""));
	return Result;
}

FString UComputeKernelSourceWithText::GetSource() const
{
	const FString StrippedEntryPoint = StripRegexCharacters(EntryPoint);
	if (StrippedEntryPoint.IsEmpty())
	{
		return SourceText;
	}

	// Search for the entry point. 
	const FRegexPattern Pattern(FString::Printf(TEXT(R"(\b%s\b\s*\()"), *StrippedEntryPoint));

	TArray<FString> Lines;
	SourceText.ParseIntoArrayLines(Lines);
	for (int32 LineIndex = 0; LineIndex < Lines.Num(); ++LineIndex)
	{
		FRegexMatcher Matcher(Pattern, Lines[LineIndex]);
		if (!Matcher.FindNext())
		{
			continue;
		}

		int32 PrevLineIndex = LineIndex - 1;
		while (PrevLineIndex >= 0 && Lines[PrevLineIndex].TrimStartAndEnd().IsEmpty())
		{
			--PrevLineIndex;
		}

		// If there is an existing numthreads semantic then we don't need to do anything.
		const bool bHasNumThreads =	PrevLineIndex >= 0 && Lines[PrevLineIndex].TrimStartAndEnd().StartsWith(TEXT("[numthreads"), ESearchCase::IgnoreCase);
		if (bHasNumThreads)
		{
			break;
		}

		// Add the numthreads semantic and return the modified source.
		const FString NumThreads = FString::Printf(TEXT("[numthreads(%d, %d, %d)]\n"), GroupSize.X, GroupSize.Y, GroupSize.Z);
		Lines.Insert(NumThreads, LineIndex);
		return FString::Join(Lines, TEXT("\n"));
	}

	return SourceText;
}

#endif