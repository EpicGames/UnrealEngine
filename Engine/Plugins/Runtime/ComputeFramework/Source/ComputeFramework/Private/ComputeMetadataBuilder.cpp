// Copyright Epic Games, Inc. All Rights Reserved.

#include "ComputeFramework/ComputeMetadataBuilder.h"

#include "ShaderParameterMetadataBuilder.h"
#include "ComputeFramework/ShaderParamTypeDefinition.h"

template<typename T>
void ParametrizedAddParm(FShaderParametersMetadataBuilder& InOutBuilder, const TCHAR* InName)
{
	InOutBuilder.AddParam<T>(InName);
}

void ComputeFramework::AddParamForType(FShaderParametersMetadataBuilder& InOutBuilder, TCHAR const* InName, FShaderValueTypeHandle const& InValueType, TArray<FShaderParametersMetadata*>& OutNestedStructs)
{
	using AddParamFuncType = TFunction<void(FShaderParametersMetadataBuilder&, const TCHAR*)>;

	static TMap<FShaderValueType, AddParamFuncType> AddParamFuncs =
	{
		{*FShaderValueType::Get(EShaderFundamentalType::Bool), &ParametrizedAddParm<bool>},
		{*FShaderValueType::Get(EShaderFundamentalType::Int), &ParametrizedAddParm<int32>},
		{*FShaderValueType::Get(EShaderFundamentalType::Int, 2), &ParametrizedAddParm<FIntPoint>},
		{*FShaderValueType::Get(EShaderFundamentalType::Int, 3), &ParametrizedAddParm<FIntVector>},
		{*FShaderValueType::Get(EShaderFundamentalType::Int, 4), &ParametrizedAddParm<FIntVector4>},
		{*FShaderValueType::Get(EShaderFundamentalType::Uint), &ParametrizedAddParm<uint32>},
		{*FShaderValueType::Get(EShaderFundamentalType::Uint, 2), &ParametrizedAddParm<FUintVector2>},
		{*FShaderValueType::Get(EShaderFundamentalType::Uint, 4), &ParametrizedAddParm<FUintVector4>},
		{*FShaderValueType::Get(EShaderFundamentalType::Float), &ParametrizedAddParm<float>},
		{*FShaderValueType::Get(EShaderFundamentalType::Float, 2), &ParametrizedAddParm<FVector2f>},
		{*FShaderValueType::Get(EShaderFundamentalType::Float, 3), &ParametrizedAddParm<FVector3f>},
		{*FShaderValueType::Get(EShaderFundamentalType::Float, 4), &ParametrizedAddParm<FVector4f>},
		{*FShaderValueType::Get(EShaderFundamentalType::Float, 4, 4), &ParametrizedAddParm<FMatrix44f>},
	};

	static TArray<FString> AllElementNames;
	if (InValueType->bIsDynamicArray)
	{
		// both struct array and normal array are treated the same
		InOutBuilder.AddRDGBufferSRV(InName, TEXT("StructuredBuffer"));
	}
	else if (InValueType->Type == EShaderFundamentalType::Struct)
	{
		FShaderParametersMetadataBuilder NestedStructBuilder;
	
		for (const FShaderValueType::FStructElement& Element : InValueType->StructElements)
		{
			AllElementNames.Add(Element.Name.ToString());
			AddParamForType(NestedStructBuilder, *AllElementNames.Last(), Element.Type, OutNestedStructs);
		}
	
		FShaderParametersMetadata* ShaderParameterMetadata = NestedStructBuilder.Build(FShaderParametersMetadata::EUseCase::ShaderParameterStruct, InName);
	
		InOutBuilder.AddNestedStruct(InName, ShaderParameterMetadata);
	
		OutNestedStructs.Add(ShaderParameterMetadata);
	}
	else if (const AddParamFuncType* Entry = AddParamFuncs.Find(*InValueType))
	{
		(*Entry)(InOutBuilder, InName);
	}
}

ComputeFramework::FTypeMetaData::FTypeMetaData(FShaderValueTypeHandle InType)
{
	FShaderParametersMetadataBuilder Builder;
	TArray<FShaderParametersMetadata*> NestedStructs;
	AddParamForType(Builder, TEXT("Dummy"), InType, NestedStructs);
	FShaderParametersMetadata* ShaderParameterMetadata = Builder.Build(FShaderParametersMetadata::EUseCase::ShaderParameterStruct, TEXT("Dummy"));
	Metadata = ShaderParameterMetadata->GetMembers()[0].GetStructMetadata();
	AllocatedMetadatas.Add(ShaderParameterMetadata);
	AllocatedMetadatas.Append(NestedStructs);
}

ComputeFramework::FTypeMetaData::~FTypeMetaData()
{
	for (FShaderParametersMetadata* Allocated: AllocatedMetadatas)
	{
		delete Allocated;
	}
}
