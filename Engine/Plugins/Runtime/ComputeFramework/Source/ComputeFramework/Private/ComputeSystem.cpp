// Copyright Epic Games, Inc. All Rights Reserved.

#include "ComputeFramework/ComputeSystem.h"

#include "ComputeFramework/ComputeGraphWorker.h"
#include "ComputeFramework/ComputeViewExtension.h"

void FComputeFrameworkSystem::CreateWorkers(FSceneInterface const* InScene, TArray<IComputeTaskWorker*>& OutWorkers)
{
	FComputeGraphTaskWorker* ComputeWorker = new FComputeGraphTaskWorker();
	ComputeWorkers.Add(InScene, ComputeWorker);
	OutWorkers.Add(ComputeWorker);

	if (UWorld* World = InScene->GetWorld())
	{
		ViewExtensions.Add(InScene, FSceneViewExtensions::NewExtension<FComputeViewExtension>(World));
	}
}

void FComputeFrameworkSystem::DestroyWorkers(FSceneInterface const* InScene, TArray<IComputeTaskWorker*>& InOutWorkers)
{
	FComputeGraphTaskWorker** Found = ComputeWorkers.Find(InScene);
	if (Found != nullptr)
	{
		int32 ArrayIndex = InOutWorkers.Find(*Found);
		if (ArrayIndex != INDEX_NONE)
		{
			InOutWorkers.RemoveAtSwap(ArrayIndex, EAllowShrinking::No);
		}

		ComputeWorkers.Remove(InScene);
	}

	ViewExtensions.Remove(InScene);
}

FComputeGraphTaskWorker* FComputeFrameworkSystem::GetComputeWorker(FSceneInterface const* InScene) const
{
	FComputeGraphTaskWorker* const* Found = ComputeWorkers.Find(InScene);
	return Found != nullptr ? *Found : nullptr;
}
