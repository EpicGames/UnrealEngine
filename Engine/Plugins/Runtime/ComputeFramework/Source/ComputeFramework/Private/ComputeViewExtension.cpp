// Copyright Epic Games, Inc. All Rights Reserved.

#include "ComputeFramework/ComputeViewExtension.h"

#include "ComputeWorkerInterface.h"

FComputeViewExtension::FComputeViewExtension(const FAutoRegister& AutoReg, UWorld* InWorld)
	: FWorldSceneViewExtension(AutoReg, InWorld)
{
}

void FComputeViewExtension::PostTLASBuild_RenderThread(FRDGBuilder& GraphBuilder, FSceneView& InView)
{
	const TWeakObjectPtr<UWorld> WorldPtr = GetWorld();
	if (!WorldPtr.IsValid())
	{
		return;
	}

	const FSceneInterface* Scene = WorldPtr->Scene;
	if (!Scene)
	{
		return;
	}

	const ERHIFeatureLevel::Type FeatureLevel = Scene->GetFeatureLevel();

	ComputeWorkers.Empty(ComputeWorkers.Num());
	Scene->GetComputeTaskWorkers(ComputeWorkers);

	for (IComputeTaskWorker* ComputeTaskWorker : ComputeWorkers)
	{
		if (ComputeTaskWorker->HasWork(ComputeTaskExecutionGroup::PostTLASBuild))
		{
			FComputeContext Context
			{
				.GraphBuilder = GraphBuilder,
				.ExecutionGroupName = ComputeTaskExecutionGroup::PostTLASBuild,
				.FeatureLevel = FeatureLevel,
				.Scene = Scene,
				.View = &InView,
			};

			ComputeTaskWorker->SubmitWork(Context);
		}
	}
}

ESceneViewExtensionFlags FComputeViewExtension::GetFlags() const
{
	const TWeakObjectPtr<UWorld> WorldPtr = GetWorld();

	if (const FSceneInterface* Scene = WorldPtr.IsValid() ? WorldPtr->Scene : nullptr)
	{
		ComputeWorkers.Empty(ComputeWorkers.Num());
		Scene->GetComputeTaskWorkers(ComputeWorkers);

		for (IComputeTaskWorker* ComputeTaskWorker : ComputeWorkers)
		{
			if (ComputeTaskWorker->HasWork(ComputeTaskExecutionGroup::PostTLASBuild))
			{
				return ESceneViewExtensionFlags::SubscribesToPostTLASBuild | ESceneViewExtensionFlags::RequiresHardwareInlineRayTracing;
			}
		}
	}

	return ESceneViewExtensionFlags::None;
}
