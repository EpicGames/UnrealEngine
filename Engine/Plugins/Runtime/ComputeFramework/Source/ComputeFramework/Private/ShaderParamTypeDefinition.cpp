// Copyright Epic Games, Inc. All Rights Reserved.

#include "ComputeFramework/ShaderParamTypeDefinition.h"

#include "ShaderParameterMetadataBuilder.h"
#include "Algo/Find.h"
#include "ComputeFramework/ComputeMetadataBuilder.h"
#include "ComputeFramework/ComputeFrameworkObjectVersion.h"
#include "Internationalization/Regex.h"
#include "Misc/DefaultValueHelper.h"
#include "Serialization/Archive.h"
#include "Templates/TypeHash.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ShaderParamTypeDefinition)

// Storage for shared shader value types.
static uint32 GetTypeHash(const FShaderValueTypeHandle& InTypeHandle)
{
	return GetTypeHash(*InTypeHandle.ValueTypePtr);
}

struct HandleKeyFuncs : BaseKeyFuncs<FShaderValueTypeHandle,FShaderValueTypeHandle,false>
{
	static KeyInitType GetSetKey(ElementInitType Element)
	{
		return Element;
	}

	static bool Matches(KeyInitType A, KeyInitType B)
	{
		// The handle value can never be null here.
		return *A.ValueTypePtr == *B.ValueTypePtr;
	}

	static uint32 GetKeyHash(KeyInitType Key)
	{
		return GetTypeHash(*Key.ValueTypePtr);
	}
};

static TSet<FShaderValueTypeHandle, HandleKeyFuncs> GloballyKnownValueTypes;
static FRWLock GloballyKnownValueTypesLock;


bool FShaderValueTypeHandle::Serialize(FArchive& Ar)
{
	Ar << *this;
	return true;
}


FShaderValueTypeHandle FShaderValueType::Get(EShaderFundamentalType InType)
{
	if (InType == EShaderFundamentalType::Struct)
	{
		return {};
	}

	FShaderValueType ValueType;
	ValueType.Type = InType;
	ValueType.DimensionType = EShaderFundamentalDimensionType::Scalar;

	return GetOrCreate(MoveTemp(ValueType));
}

FShaderValueTypeHandle FShaderValueType::Get(EShaderFundamentalType InType, int32 InElemCount)
{
	if (InType == EShaderFundamentalType::Struct || InElemCount < 1 || InElemCount > 4)
	{
		return {};
	}

	FShaderValueType ValueType;
	ValueType.Type = InType;
	ValueType.DimensionType = EShaderFundamentalDimensionType::Vector;
	ValueType.VectorElemCount = InElemCount;

	return GetOrCreate(MoveTemp(ValueType));
}

FShaderValueTypeHandle FShaderValueType::Get(EShaderFundamentalType InType, int32 InRowCount, int32 InColumnCount)
{
	if (InType == EShaderFundamentalType::Struct || 
		InRowCount < 1 || InRowCount > 4 ||
		InColumnCount < 1 || InColumnCount > 4)
	{
		return {};
	}

	FShaderValueType ValueType;
	ValueType.Type = InType;
	ValueType.DimensionType = EShaderFundamentalDimensionType::Matrix;
	ValueType.MatrixRowCount = InRowCount;
	ValueType.MatrixColumnCount = InColumnCount;

	return GetOrCreate(MoveTemp(ValueType));
}

FShaderValueTypeHandle FShaderValueType::Get(
	FName InName, 
	std::initializer_list<FStructElement> InStructElements)
{
	if (InName == NAME_None)
	{
		return {};
	}

	// TODO: Check if the name and the element names are valid HLSL identifiers 
	// (i.e. identifier characters only, not a reserved keyword, and no duplicates).
	// TODO: Check if the name matches another struct with a different layout.

	FShaderValueType ValueType;
	ValueType.Name = InName;
	ValueType.Type = EShaderFundamentalType::Struct;
	ValueType.DimensionType = EShaderFundamentalDimensionType::Scalar;

	for (const FStructElement &StructElement : InStructElements)
	{
		// FIXME: We don't allow nested structs for now to avoid complicating the 
		// GetTypeDeclaration call too much.
		if (StructElement.Type.ValueTypePtr == nullptr || 
		    StructElement.Type.ValueTypePtr->Type == EShaderFundamentalType::Struct)
		{
			return {};
		}

		ValueType.StructElements.Add(StructElement);
	}

	// We don't allow empty structs.
	if (ValueType.StructElements.IsEmpty())
	{
		return {};
	}

	return GetOrCreate(MoveTemp(ValueType));
}

FShaderValueTypeHandle FShaderValueType::Get(FName InName, const TArray<FStructElement>& InStructElements)
{
	if (InName == NAME_None)
	{
		return {};
	}

	// TODO: Check if the name and the element names are valid HLSL identifiers 
	// (i.e. identifier characters only, not a reserved keyword, and no duplicates).
	// TODO: Check if the name matches another struct with a different layout.

	FShaderValueType ValueType;
	ValueType.Name = InName;
	ValueType.Type = EShaderFundamentalType::Struct;
	ValueType.DimensionType = EShaderFundamentalDimensionType::Scalar;

	for (const FStructElement &StructElement : InStructElements)
	{
		ValueType.StructElements.Add(StructElement);
	}

	// We don't allow empty structs.
	if (ValueType.StructElements.IsEmpty())
	{
		return {};
	}

	return GetOrCreate(MoveTemp(ValueType));
}

FShaderValueTypeHandle FShaderValueType::MakeDynamicArrayType(const FShaderValueTypeHandle& InElementType)
{
	FShaderValueType Type = *InElementType;

	Type.bIsDynamicArray = true;

	return GetOrCreate(MoveTemp(Type));
}

FShaderValueTypeHandle FShaderValueType::FromString(const FString& InTypeDecl)
{
	static const FRegexPattern ValueTypeDeclPattern(TEXT(R"(\s*(bool|int|uint|float)((?:[1-4])|(?:[1-4]x[1-4]))?\s*)"));

	// We really should have a FStringView version of the regex matcher. This level of string
	// copying is gross.
	FRegexMatcher Matcher(ValueTypeDeclPattern, InTypeDecl);
	if (!Matcher.FindNext())
	{
		return {};
	}

	FString FundamentalTypeStr = Matcher.GetCaptureGroup(1);
	FString DimensionTypeStr   = Matcher.GetCaptureGroup(2);

	EShaderFundamentalType FundamentalType;
	if (FundamentalTypeStr == TEXT("bool"))
	{
		FundamentalType = EShaderFundamentalType::Bool;
	}
	else if (FundamentalTypeStr == TEXT("int"))
	{
		FundamentalType = EShaderFundamentalType::Int;
	}
	else if (FundamentalTypeStr == TEXT("uint"))
	{
		FundamentalType = EShaderFundamentalType::Uint;
	}
	else if (FundamentalTypeStr == TEXT("float"))
	{
		FundamentalType = EShaderFundamentalType::Float;
	}
	else
	{
		return {};
	}

	if (DimensionTypeStr.Len() == 0)
	{
		return Get(FundamentalType);
	}
	else if (DimensionTypeStr.Len() == 1)
	{
		int32 Dim = 0;
		if (FDefaultValueHelper::ParseInt(DimensionTypeStr, Dim))
		{
			return Get(FundamentalType, Dim);
		}
	}
	else if (DimensionTypeStr.Len() == 3)
	{
		int32 Row = 0, Col = 0;
		if (FDefaultValueHelper::ParseInt(DimensionTypeStr.Left(1), Row) &&
			FDefaultValueHelper::ParseInt(DimensionTypeStr.Right(1), Col))
		{
			return Get(FundamentalType, Row, Col);
		}
	}

	// Failure to parse.
	return {};
}


FShaderValueTypeHandle FShaderValueType::GetOrCreate(FShaderValueType&& InValueType)
{
	FShaderValueTypeHandle Handle;
	Handle.ValueTypePtr = &InValueType;

	{
		FReadScopeLock Lock(GloballyKnownValueTypesLock);
	
		if (FShaderValueTypeHandle* FoundHandle = GloballyKnownValueTypes.Find(Handle))
		{
			return *FoundHandle;
		}
	}

	FWriteScopeLock Lock(GloballyKnownValueTypesLock);

	if (FShaderValueTypeHandle* FoundHandle = GloballyKnownValueTypes.Find(Handle))
	{
		return *FoundHandle;
	}

	Handle.ValueTypePtr = new FShaderValueType(MoveTemp(InValueType));
	GloballyKnownValueTypes.Add(Handle);
	return Handle;
}

bool FShaderValueType::operator==(const FShaderValueType& InOtherType) const
{
	if (Type != InOtherType.Type)
	{
		return false;
	}

	if (bIsDynamicArray != InOtherType.bIsDynamicArray)
	{
		return false;
	}
	
	if (Type == EShaderFundamentalType::Struct)
	{
		if (Name != InOtherType.Name || StructElements.Num() != InOtherType.StructElements.Num())
		{
			return false;
		}

		for (int32 Index = 0; Index < StructElements.Num(); Index++)
		{
			if (StructElements[Index] != InOtherType.StructElements[Index])
			{
				return false;
			}
		}
	}
	else
	{
		if (DimensionType != InOtherType.DimensionType)
		{
			return false;
		}

		if (DimensionType == EShaderFundamentalDimensionType::Vector && 
		    VectorElemCount != InOtherType.VectorElemCount)
		{
			return false;
		}
		else if (DimensionType == EShaderFundamentalDimensionType::Matrix &&
		    (MatrixRowCount != InOtherType.MatrixRowCount || 
			 MatrixColumnCount != InOtherType.MatrixColumnCount))
		{
			return false;
		}
	}

	return true;
}

uint32 GetTypeHash(const FShaderValueType& InShaderValueType)
{
	uint32 Hash = GetTypeHash(int32(InShaderValueType.Type));

	if (InShaderValueType.Type == EShaderFundamentalType::Struct)
	{
		Hash = HashCombine(Hash, GetTypeHash(InShaderValueType.Name));
		Hash = HashCombine(Hash, GetTypeHash(InShaderValueType.StructElements.Num()));
		for (const FShaderValueType::FStructElement& StructElement : InShaderValueType.StructElements)
		{
			Hash = HashCombine(Hash, GetTypeHash(StructElement.Name));
			Hash = HashCombine(Hash, GetTypeHash(*StructElement.Type.ValueTypePtr));
		}
	}
	else
	{
		Hash = HashCombine(Hash, GetTypeHash(int32(InShaderValueType.DimensionType)));

		if (InShaderValueType.DimensionType == EShaderFundamentalDimensionType::Vector)
		{
			Hash = HashCombine(Hash, GetTypeHash(int32(InShaderValueType.VectorElemCount)));
		}
		else if (InShaderValueType.DimensionType == EShaderFundamentalDimensionType::Matrix)
		{
			Hash = HashCombine(Hash, GetTypeHash(int32(InShaderValueType.MatrixRowCount)));
			Hash = HashCombine(Hash, GetTypeHash(int32(InShaderValueType.MatrixColumnCount)));
		}
	}
	
	Hash = HashCombine(Hash, GetTypeHash(InShaderValueType.bIsDynamicArray));
	return Hash;
}

FString FShaderValueType::ToString(const FName& InStructTypeNameOverride) const
{
	// FIXME: Cache on create?
	FString BaseName;
	switch (Type)
	{
	case EShaderFundamentalType::Bool:
		BaseName = "bool";
		break;
	case EShaderFundamentalType::Int:
		BaseName = "int";
		break;
	case EShaderFundamentalType::Uint:
		BaseName = "uint";
		break;
	case EShaderFundamentalType::Float:
		BaseName = "float";
		break;

	case EShaderFundamentalType::Struct:
		BaseName = (InStructTypeNameOverride != NAME_None) ? InStructTypeNameOverride.ToString() : Name.ToString();
		break;
	}

	if (Type != EShaderFundamentalType::Struct)
	{
		if (DimensionType == EShaderFundamentalDimensionType::Vector)
		{
			BaseName.Appendf(TEXT("%d"), VectorElemCount);
		}
		else if (DimensionType == EShaderFundamentalDimensionType::Matrix)
		{
			BaseName.Appendf(TEXT("%dx%d"), MatrixRowCount, MatrixColumnCount);
		}
	}

	if (bIsDynamicArray)
	{
		BaseName = FString::Printf(TEXT("StructuredBuffer<%s>"), *BaseName);
	}
	
	return BaseName;
}

FString FShaderValueType::GetTypeDeclaration(const TMap<FName, FName>& InTypeNamesToReplace, bool bCommentPaddings) const
{
	// FIXME: Cache on create?
	if (Type != EShaderFundamentalType::Struct)
	{
		return {};
	}

	FShaderValueTypeHandle TypeHandle;
	TypeHandle.ValueTypePtr = this;
	
	if (bIsDynamicArray)
	{
		FShaderValueType ElementType = *this;
		ElementType.bIsDynamicArray = false;
		TypeHandle = GetOrCreate(MoveTemp(ElementType));
	}
	
	ComputeFramework::FTypeMetaData TypeMetaData(TypeHandle);
	const TArray<FShaderParametersMetadata::FMember>& MemberDatas = TypeMetaData.Metadata->GetMembers();
	
	if (!ensure(MemberDatas.Num() == StructElements.Num()))
	{
		return {};
	}

	// Record where each padding starts	
	TArray<int32> PadOffsets;
	PadOffsets.Reserve(MemberDatas.Num());
	
	// Total allocated size includes member size + per member padding
	TArray<int32> MemberTotalSizes;
	MemberTotalSizes.Reserve(MemberDatas.Num());
	
	for (int32 Index = 0; Index < MemberDatas.Num(); Index++)
	{
		if (Index + 1 < MemberDatas.Num())
		{
			MemberTotalSizes.Add(MemberDatas[Index + 1].GetOffset() - MemberDatas[Index].GetOffset());
		}
		else
		{
			MemberTotalSizes.Add(TypeMetaData.Metadata->GetSize() - MemberDatas[Index].GetOffset());
		}

		// Initialize the pad offsets to be the member offset
		PadOffsets.Add(MemberDatas[Index].GetOffset());
	}

	// Calculate the padding size, should be in multiples of 4
	TArray<int32> Pads;
	Pads.Reserve(MemberDatas.Num());
	for (int32 Index = 0; Index < StructElements.Num(); Index++)
	{
		const FStructElement& StructElement = StructElements[Index];
		int32 InlineSize = 0;
		if (StructElement.Type->bIsDynamicArray)
		{
			// Array pointers are always of size SHADER_PARAMETER_POINTER_ALIGNMENT
			InlineSize = SHADER_PARAMETER_POINTER_ALIGNMENT;
		}
		else if (StructElement.Type->StructElements.Num() > 0)
		{
			// Struct type does not need external padding
			// it should be padded internally to align to 16 byte boundary 
			InlineSize = MemberTotalSizes[Index];
		}
		else
		{
			InlineSize = StructElement.Type->GetResourceElementSize();
		}

		// Padding starting position = Member starting position + member size;
		PadOffsets[Index] += InlineSize;
		
		int32 Pad = MemberTotalSizes[Index] - InlineSize;
		Pads.Add(Pad);
	}

	TArray<FString> Elements;
	Elements.Reserve(StructElements.Num());
	for (int32 Index = 0; Index < StructElements.Num(); Index++)
	{
		const FStructElement& StructElement = StructElements[Index];
		
		FName MemberTypeNameToUse = StructElement.Type->Name;
		if (const FName* Replacement = InTypeNamesToReplace.Find(StructElement.Type->Name))
		{
			MemberTypeNameToUse = *Replacement;
		}
		
		Elements.Add(FString::Printf(TEXT("    %s %s;\n"), *StructElement.Type.ValueTypePtr->ToString(MemberTypeNameToUse), *StructElement.Name.ToString()));

		// Add padding elements
		if (Pads[Index] > 0)
		{
			if (ensure(Pads[Index] % 4 == 0))
			{
				const int32 NumPads = Pads[Index] / 4;

				const FString CommentPrefix = bCommentPaddings ? TEXT("// ") : TEXT("");
				
				if (NumPads == 1)
				{
					Elements.Add(FString::Printf(TEXT("    %sfloat Padding_%s_%d;\n"), *CommentPrefix, *StructElement.Name.ToString(), PadOffsets[Index]));
				}
				else
				{
					Elements.Add(FString::Printf(TEXT("    %sfloat Padding_%s_%d[%d];\n"), *CommentPrefix, *StructElement.Name.ToString(), PadOffsets[Index], NumPads));
				}
			}
		}
	}

	FName TypeNameToUse = Name;
	if (const FName* Replacement = InTypeNamesToReplace.Find(Name))
	{
		TypeNameToUse = *Replacement;
	}	

	return FString::Printf(TEXT("struct %s {\n%s}"), 
		*TypeNameToUse.ToString(),
		*FString::Join(Elements, TEXT("")));
}

TArray<FShaderValueTypeHandle> FShaderValueType::GetMemberStructTypes() const
{
	struct FStructCollector
	{
		void WalkMembers(const FShaderValueType& InType, TArray<FShaderValueTypeHandle>& OutResults)
		{
			for (const FStructElement& StructElement : InType.StructElements)
			{
				if (StructElement.Type->Type == EShaderFundamentalType::Struct)
				{
					if (!StructsSeen.Contains(StructElement.Type->Name))
					{
						WalkMembers(*StructElement.Type, OutResults);
						OutResults.Add(StructElement.Type);
						StructsSeen.Add(StructElement.Type->Name);
					}
				}
			}
		};

		TSet<FName> StructsSeen;
	};

	TArray<FShaderValueTypeHandle> Results;
	Results.Reserve(4);
	
	FStructCollector Collector;
	Collector.WalkMembers(*this, Results);
	
	return Results;
}


int32 FShaderValueType::GetResourceElementSize() const
{
	int32 Size = 0;
	
	switch (Type)
	{
	case EShaderFundamentalType::Bool:
	case EShaderFundamentalType::Int:
	case EShaderFundamentalType::Uint:
	case EShaderFundamentalType::Float:
		Size = 4;		// Yes, even for bool.
		break;
	case EShaderFundamentalType::Struct:
		for (const FStructElement& Elem: StructElements)
		{
			Size += Elem.Type->GetResourceElementSize();
		}
		break;
	}

	switch(DimensionType)
	{
	case EShaderFundamentalDimensionType::Scalar:
		break;
	case EShaderFundamentalDimensionType::Vector:
		Size *= VectorElemCount;
		break;
	case EShaderFundamentalDimensionType::Matrix:
		Size *= MatrixRowCount * MatrixColumnCount;
		break;
	}

	return Size;
}


FString FShaderValueType::GetZeroValueAsString() const
{
	FString FundamentalZeroConstant;
	switch(Type)
	{
	case EShaderFundamentalType::None:
		checkNoEntry();
		break;
	case EShaderFundamentalType::Bool:
		FundamentalZeroConstant = TEXT("false");
		break;
	case EShaderFundamentalType::Int:
	case EShaderFundamentalType::Uint:
		FundamentalZeroConstant = TEXT("0");
		break;
	case EShaderFundamentalType::Float:
		FundamentalZeroConstant = TEXT("0.0f");
		break;
	case EShaderFundamentalType::Struct:
		checkf(Type != EShaderFundamentalType::Struct, TEXT("Structs not supported yet.")); //-V547
		break;
	}

	int32 ValueCount = 0;
	switch(DimensionType)
	{
	case EShaderFundamentalDimensionType::Scalar:
		ValueCount = 1;
		break;
		
	case EShaderFundamentalDimensionType::Vector:
		ValueCount = VectorElemCount;
		break;
		
	case EShaderFundamentalDimensionType::Matrix:
		ValueCount = MatrixRowCount * MatrixColumnCount;
		break;
	}

	TArray<FString> ValueArray;
	ValueArray.Init(FundamentalZeroConstant, ValueCount);

	return FString::Printf(TEXT("%s(%s)"), *ToString(), *FString::Join(ValueArray, TEXT(", ")));
}


FArchive& operator<<(FArchive& InArchive, FShaderValueTypeHandle& InHandle)
{
	FShaderValueType ValueTypeTemp;
	FShaderValueType *ValueTypePtr;

	InArchive.UsingCustomVersion(FComputeFrameworkObjectVersion::GUID);
	
	if (InArchive.IsLoading())
	{
		ValueTypePtr = &ValueTypeTemp;
	}
	else if (InHandle.ValueTypePtr)
	{
		ValueTypePtr = const_cast<FShaderValueType*>(InHandle.ValueTypePtr);
	}
	else
	{
		ValueTypeTemp.Type = EShaderFundamentalType::None;
		ValueTypePtr = &ValueTypeTemp;
	}

	InArchive << ValueTypePtr->Type;

	// When the object version is added, bIsDynamicArray is also added
	if (InArchive.CustomVer(FComputeFrameworkObjectVersion::GUID) >= FComputeFrameworkObjectVersion::InitialVersion)
	{
		InArchive << ValueTypePtr->bIsDynamicArray;
	}
		
	if (ValueTypePtr->Type == EShaderFundamentalType::Struct)
	{
		InArchive << ValueTypePtr->Name;
		InArchive << ValueTypePtr->StructElements;
	}
	else
	{
		InArchive << ValueTypePtr->DimensionType;

		if (ValueTypePtr->DimensionType == EShaderFundamentalDimensionType::Vector)
		{
			InArchive << ValueTypePtr->VectorElemCount;
		}
		else if (ValueTypePtr->DimensionType == EShaderFundamentalDimensionType::Matrix)
		{
			InArchive << ValueTypePtr->MatrixRowCount;
			InArchive << ValueTypePtr->MatrixColumnCount;
		}
	}

	if (InArchive.IsLoading())
	{
		InHandle = FShaderValueType::GetOrCreate(MoveTemp(ValueTypeTemp));
	}

	return InArchive;
}

FArchive& operator<<(FArchive& InArchive, FShaderValueType::FStructElement& InElement)
{
	InArchive << InElement.Name;
	InArchive << InElement.Type;
	return InArchive;
}

using FResourceStingPair = TPair<EShaderResourceType, FString>;

static FResourceStingPair ResTypeStringMap[] = {
	FResourceStingPair(EShaderResourceType::Texture1D,			TEXT("Texture1D")),
	FResourceStingPair(EShaderResourceType::Texture2D,			TEXT("Texture2D")),
	FResourceStingPair(EShaderResourceType::Texture3D,			TEXT("Texture3D")),
	FResourceStingPair(EShaderResourceType::TextureCube,		TEXT("TextureCube")),
	FResourceStingPair(EShaderResourceType::StructuredBuffer,	TEXT("StructuredBuffer")),
	FResourceStingPair(EShaderResourceType::ByteAddressBuffer,	TEXT("ByteAddressBuffer")),
	FResourceStingPair(EShaderResourceType::Buffer,				TEXT("Buffer")),
	};


EShaderResourceType FShaderParamTypeDefinition::ParseResource(const FString& Str)
{
	for (auto& Pair : ResTypeStringMap)
	{
		if (Str.Contains(Pair.Value))
		{
			return Pair.Key;
		}
	}

	return EShaderResourceType::None;
}

void FShaderParamTypeDefinition::ResetTypeDeclaration(
	)
{
	FString TypeDecl;
	if (BindingType == EShaderParamBindingType::ReadWriteResource)
	{
		TypeDecl.Append(TEXT("RW"));
	}

	const bool bIsResourceType = BindingType == EShaderParamBindingType::ReadOnlyResource || BindingType == EShaderParamBindingType::ReadWriteResource;
	if (bIsResourceType)
	{
		auto* foundItem = Algo::FindByPredicate(
			ResTypeStringMap, 
			[this](const FResourceStingPair& Pair) 
			{ 
				return Pair.Key == ResourceType; 
			});
		check(foundItem);

		TypeDecl.Appendf(TEXT("%s<"), *foundItem->Value);
	}

	TypeDecl.Append(ValueType->ToString());

	if (bIsResourceType)
	{
		TypeDecl.AppendChar(TEXT('>'));
	}

	TypeDeclaration = MoveTemp(TypeDecl);
}


FShaderFunctionDefinition& FShaderFunctionDefinition::SetName(FString InName)
{
	Name = InName;
	return *this;
}

FShaderFunctionDefinition& FShaderFunctionDefinition::AddParam(FShaderValueTypeHandle InValueType, EShaderParamModifier InModifier)
{
	FShaderParamTypeDefinition Def;
	Def.ValueType = InValueType;
	Def.Modifier = InModifier;
	Def.ResetTypeDeclaration();
	ParamTypes.Add(Def);
	return *this;
}

FShaderFunctionDefinition& FShaderFunctionDefinition::AddParam(EShaderFundamentalType InType, int32 InRowCount, int32 InColumnCount, EShaderParamModifier InModifier)
{
	FShaderParamTypeDefinition Def;
	Def.Modifier = InModifier;

	if (InRowCount > 0 && InColumnCount > 0)
	{
		Def.ValueType = FShaderValueType::Get(InType, InRowCount, InColumnCount);
	}
	else if (InRowCount > 0)
	{
		Def.ValueType = FShaderValueType::Get(InType, InRowCount);
	}
	else
	{
		Def.ValueType = FShaderValueType::Get(InType);
	}

	Def.ResetTypeDeclaration();

	ParamTypes.Add(Def);
	return *this;
}

FShaderFunctionDefinition& FShaderFunctionDefinition::AddReturnType(FShaderValueTypeHandle InValueType)
{
	// Only one return type allowed.
	ensure(!bHasReturnType);
	bHasReturnType = true;

	// Add as a param.
	AddParam(InValueType);

	// Return type is expected to be the first param.
	for (int I = ParamTypes.Num() - 1; I > 0; --I)
	{
		ParamTypes.Swap(I, I - 1);
	}
	return *this;
}

FShaderFunctionDefinition& FShaderFunctionDefinition::AddReturnType(EShaderFundamentalType InType, int32 InRowCount, int32 InColumnCount)
{
	// Only one return type allowed.
	ensure(!bHasReturnType);
	bHasReturnType = true;

	// Add as a param.
	AddParam(InType, InRowCount, InColumnCount);

	// Return type is expected to be the first param.
	for (int I = ParamTypes.Num() - 1; I > 0; --I)
	{
		ParamTypes.Swap(I, I - 1);
	}
	return *this;
}
