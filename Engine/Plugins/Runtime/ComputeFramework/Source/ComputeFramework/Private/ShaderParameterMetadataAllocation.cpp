// Copyright Epic Games, Inc. All Rights Reserved.

#include "ComputeFramework/ShaderParameterMetadataAllocation.h"
#include "ShaderParameterMetadata.h"

FShaderParametersMetadataAllocations::~FShaderParametersMetadataAllocations()
{
	for (FShaderParametersMetadata* ShaderParameterMetadata : ShaderParameterMetadatas)
	{
		delete ShaderParameterMetadata;
	}
}
