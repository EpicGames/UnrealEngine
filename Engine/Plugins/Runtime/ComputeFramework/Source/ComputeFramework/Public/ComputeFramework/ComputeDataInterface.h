// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "ComputeDataInterface.generated.h"

#define UE_API COMPUTEFRAMEWORK_API

struct FComputeKernelDefinitionSet;
struct FComputeKernelPermutationVector;
struct FShaderFunctionDefinition;
struct FShaderParametersMetadataAllocations;
class FShaderParametersMetadataBuilder;
class UComputeDataProvider;

/** 
 * Compute Data Interface required to compile a Compute Graph. 
 * Compute Kernels require Data Interfaces to fulfill their external functions.
 * Compute Data Interfaces define how Compute Data Providers will actually marshal data in and out of Kernels.
 */
UCLASS(MinimalAPI, Abstract, Const)
class UComputeDataInterface : public UObject
{
	GENERATED_BODY()

public:
	/** Get a name for referencing the data interface its shader parameter structures. The InUniqueIndex is optional for use in generating the name. */
	virtual TCHAR const* GetClassName() const PURE_VIRTUAL(UComputeDataInterface::GetClassName, return nullptr;)
	/** Does the associated UComputeDataProvider provide invocations and thread counts. One and only one data interface per kernel should return true. */
	virtual bool IsExecutionInterface() const { return false; }
	/** Return true if the associated UComputeDataProvider holds data that can be combined into a single dispatch invocation. */
	virtual bool CanSupportUnifiedDispatch() const { return false; }
	/** Gather compile definitions from the data interface. Any connected kernel will compile with these. */
	virtual void GetDefines(FComputeKernelDefinitionSet& OutDefinitionSet) const {}
	/** Gather permutations from the data interface. Any connected kernel will include these in its total compiled permutations. */
	virtual void GetPermutations(FComputeKernelPermutationVector& OutPermutationVector) const {}
	/** Get the data interface functions available to fulfill external inputs of a kernel. */
	virtual void GetSupportedInputs(TArray<FShaderFunctionDefinition>& OutFunctions) const {}
	/** Get the data interface functions available to fulfill external outputs of a kernel. */
	virtual void GetSupportedOutputs(TArray<FShaderFunctionDefinition>& OutFunctions) const {}
	/** Gather the shader metadata exposed by the data provider payload. */
	virtual void GetShaderParameters(TCHAR const* UID, FShaderParametersMetadataBuilder& InOutBuilder, FShaderParametersMetadataAllocations& InOutAllocations) const {}
	/** If the data interface HLSL comes from a file this should fill the virtual file path. This is used to resolve shader compilation errors. */
	virtual TCHAR const* GetShaderVirtualPath() const { return nullptr; }
	/** Get a hash that changes on any data interface changes that affect kernel compilation. */
	virtual void GetShaderHash(FString& InOutKey) const {}
	/** Gather any extra struct types that this data provider relies on. */
	virtual void GetStructDeclarations(TSet<FString>& OutStructsSeen, TArray<FString>& OutStructs) const {}
	/** Gather the shader code for this data provider. */
	virtual void GetHLSL(FString& OutHLSL, FString const& InDataInterfaceName) const {}
	/** Whether a GPU->CPU readback should be scheduled after this data is used as a kernel output. */
	virtual bool GetRequiresReadback() const { return false; }
	/** Whether PreSubmit should be called when this data is used as a kernel output. */
	virtual bool GetRequiresPreSubmitCall() const { return false; }
	/** Whether PostSubmit should be called when this data is used as a kernel output. */
	virtual bool GetRequiresPostSubmitCall() const { return false; }
	/** Whether to call InitializePerKernelInputOutputMasks on the provider in addition to Initialize. */
	virtual bool GetRequiresPerKernelInputOutputMasks() const { return false; }
	/** Get class of UObject required to instantiate a UComputeDataProvider from this interface. */
	virtual UClass* GetBindingType() const { return nullptr; }
	/** Instantiate an associated UComputeDataProvider. */
	virtual UComputeDataProvider* CreateDataProvider() const { return nullptr; }

	UE_DEPRECATED(5.6, "Use the no-argument version together with UComputeDataProvider::Initialize()")
	virtual UComputeDataProvider* CreateDataProvider(TObjectPtr<UObject> InBinding, uint64 InInputMask, uint64 InOutputMask) const { return nullptr; }
};

#undef UE_API
