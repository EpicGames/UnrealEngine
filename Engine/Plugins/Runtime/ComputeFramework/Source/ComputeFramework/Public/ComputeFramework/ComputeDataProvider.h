// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "RenderGraphFwd.h"
#include "Containers/StridedView.h"
#include "ComputeDataProvider.generated.h"

class UComputeDataInterface;
class FComputeDataProviderRenderProxy;
struct FComputeKernelPermutationVector;
class FRDGBuilder;
class FRDGExternalAccessQueue;
struct FComputeContext;

/**
 * Compute Framework Data Provider.
 * A concrete instance of this is responsible for supplying data declared by a UComputeDataInterface.
 * One of these must be created for each UComputeDataInterface object in an instance of a Compute Graph.
 */
UCLASS(MinimalAPI, Abstract)
class UComputeDataProvider : public UObject
{
	GENERATED_BODY()

public:
	/** Set up the data provider from the given data interface and binding. */
	virtual void Initialize(UComputeDataInterface const* InDataInterface, UObject* InBinding, uint64 InInputMask, uint64 InOutputMask) {}
	virtual void Initialize(int32 InDataInterfaceIndex, UComputeDataInterface const* InDataInterface, UObject* InBinding, uint64 InInputMask, uint64 InOutputMask)
	{
		Initialize(InDataInterface, InBinding, InInputMask, InOutputMask);
	}


	struct FInputOutputMask
	{
		uint64 InputMask = 0;
		uint64 OutputMask = 0;
	};

	/**
	 * Called only when the associated UComputeDataInterface::GetRequiresPerKernelInputOutputMasks() returns true.
	 * Supplies the per-kernel breakdown of input/output bindings, which can be use by the proxy for bookkeeping 
	 * data related to the currently dispatching kernel (looked up via FComputeContext::GraphKernelIndex). 
	 */
	virtual void InitializePerKernelInputOutputMasks(TMap<int32, FInputOutputMask>&& InPerKernelMasks) {}

	/**
	 * Get an associated render thread proxy object.
	 * Currently these are created and destroyed per frame by the FComputeGraphInstance.
	 */
	virtual FComputeDataProviderRenderProxy* GetRenderProxy() { return nullptr; }

	/** Reset state and release any held resources. */
	virtual void Reset() {}
};

/**
 * Compute Framework Data Provider Proxy. 
 * A concrete instance of this is created by the UComputeDataProvider gathering of data for a Compute Kernel on the render thread. 
 */
class FComputeDataProviderRenderProxy
{
public:
	virtual ~FComputeDataProviderRenderProxy() = default;

	/** 
	 * Called on render thread to determine invocation count and dispatch thread counts per invocation. 
	 * This will only be called if the associated UComputeDataInterface returned true for IsExecutionInterface().
	*/
	virtual int32 GetDispatchThreadCount(TArray<FIntVector>& ThreadCounts) const { return 0; };

	/** Data needed for validation. */
	struct FValidationData
	{
		int32 NumInvocations;
		int32 ParameterStructSize;
	};

	/**
	 * Validates that we are OK to dispatch work.
	 * Default implementation returns false.
	 */
	virtual bool IsValid(FValidationData const& InValidationData) const { return false; }

	/** Data needed for setting permuations. */
	struct FPermutationData
	{
		const int32 NumInvocations;
		const FComputeKernelPermutationVector& PermutationVector;
		TArray<int32> PermutationIds;
	};

	/** 
	 * Gathers permutation bits for each invocation.
	 * This is called before any calls to AllocateResources() because we validate all requested shaders before doing any further work.
	 */
	virtual void GatherPermutations(FPermutationData& InOutPermutationData) const {}

	/** Setup needed to allocate resources. */
	struct FAllocationData
	{
		int32 NumGraphKernels = 0;
		/** Queue for AllocateResources() to add any resources that need to be externally (non-RDG) managed at the end of compute graph execution. */
		FRDGExternalAccessQueue& ExternalAccessQueue;
	};

	/* Called once before any calls to GatherDispatchData() to allow any RDG resource allocation. */
	virtual void AllocateResources(FRDGBuilder& GraphBuilder, FAllocationData const& InAllocationData) {}

	/** Setup needed to gather dispatch data. */
	struct FDispatchData
	{
		int32 GraphKernelIndex;
		int32 NumInvocations;
		bool bUnifiedDispatch;
		int32 ParameterStructSize;
		int32 ParameterBufferOffset;
		int32 ParameterBufferStride;
		uint8* ParameterBuffer;
	};

	/** Collect parameter data required to dispatch work. */
	virtual void GatherDispatchData(FDispatchData const& InDispatchData) {}

	using FReadbackCallback = TFunction<void(const void* InData, int InNumBytes)>;
	struct FReadbackData
	{
		/** The buffer to be read back. */
		FRDGBufferRef Buffer;

		/** The number of bytes to read back. */
		uint32 NumBytes;

		/** Callback to execute once data is ready for CPU consumption. */
		const FReadbackCallback* ReadbackCallback_RenderThread;
	};

	/** Data for any readbacks that should be performed. */
	virtual void GetReadbackData(TArray<FReadbackData>& OutReadbackData) const {}

	/** Called immediately prior to every kernel dispatch that outputs to this data. */
	virtual void PreSubmit(FComputeContext& Context) const {}

	/** Called immediately after every kernel dispatch that outputs to this data. */
	virtual void PostSubmit(FComputeContext& Context) const {}

protected:
	/** Helper for making an FStridedView over the FDispatchData. */
	template <typename ElementType>
	TStridedView<ElementType> MakeStridedParameterView(FDispatchData const& InDispatchData)
	{
		return TStridedView<ElementType>(InDispatchData.ParameterBufferStride, (ElementType*)(InDispatchData.ParameterBuffer + InDispatchData.ParameterBufferOffset), InDispatchData.NumInvocations);
	}

public:
	UE_DEPRECATED(5.6, "Use PostSubmit instead.")
	virtual void PostGraphDispatch(FRDGBuilder& GraphBuilder) const {}
	UE_DEPRECATED(5.7, "Use version that takes FComputeContext instead.")
	COMPUTEFRAMEWORK_API virtual void PreSubmit(FRDGBuilder& InGraphBuilder) const;
	UE_DEPRECATED(5.7, "Use version that takes FComputeContext instead.")
	COMPUTEFRAMEWORK_API virtual void PostSubmit(FRDGBuilder& InGraphBuilder) const;
};
