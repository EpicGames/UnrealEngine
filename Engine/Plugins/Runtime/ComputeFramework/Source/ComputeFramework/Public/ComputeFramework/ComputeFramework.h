// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

class FSceneInterface;
enum EShaderPlatform : uint16;

DECLARE_LOG_CATEGORY_EXTERN(LogComputeFramework, Log, All);

namespace ComputeFramework
{
	/** Returns true if ComputeFramework is supported on a platform. */
	COMPUTEFRAMEWORK_API bool IsSupported(EShaderPlatform ShaderPlatform);

	/** Returns true if ComputeFramework is currently enabled. */
	COMPUTEFRAMEWORK_API bool IsEnabled();

	/** Returns true if compute graphs are compiled on first use instead of on PostLoad(). */
	COMPUTEFRAMEWORK_API bool IsDeferredCompilation();

	/** Rebuild all compute graphs. */
	COMPUTEFRAMEWORK_API void RebuildComputeGraphs();

	/** Tick shader compilation. */
	COMPUTEFRAMEWORK_API void TickCompilation(float DeltaSeconds);

	/** Flush any enqueued ComputeGraph work for a given execution group. */
	COMPUTEFRAMEWORK_API void FlushWork(FSceneInterface const* InScene, FName InExecutionGroupName);

	/** Aborts any queued work associated with the scene / execution group / owner pointer tuple. */
	COMPUTEFRAMEWORK_API void AbortWork(FSceneInterface const* InScene, UObject* OwnerPointer);
}
