// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

/** Timing result for a single compute kernel, fulfilled by the GPU profiler's end-of-pipe callback.
 *  Currently captures busy time only; idle and wait times can be added here as needed. */
struct FComputeFrameworkKernelTiming
{
    double BusyMs = 0.0;
};
