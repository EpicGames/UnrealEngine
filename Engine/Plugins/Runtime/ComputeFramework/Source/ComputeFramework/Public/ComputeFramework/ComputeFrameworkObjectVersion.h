// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Misc/Guid.h"

#define UE_API COMPUTEFRAMEWORK_API

struct FComputeFrameworkObjectVersion
{
	// Not instantiable.
	FComputeFrameworkObjectVersion() = delete;

	enum Type
	{
		InitialVersion = 0,
		// -----<new versions can be added above this line>-------------------------------------------------
		VersionPlusOne,
		LatestVersion = VersionPlusOne - 1
	};

	// The GUID for this custom version number
	UE_API const static FGuid GUID;
};

#undef UE_API
