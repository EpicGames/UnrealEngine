// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#if WITH_EDITORONLY_DATA
#include "RHIFeatureLevel.h"
#endif

#include "ComputeGraph.generated.h"

enum EShaderPlatform : uint16;
class FArchive;
class FComputeGraphBuilder;
struct FComputeKernelCompileResults;
struct FComputeKernelDefinitionSet;
struct FComputeKernelPermutationVector;
class FComputeKernelResource;
class FComputeGraphRenderProxy;
class ITargetPlatform;
class FShaderParametersMetadata;
struct FShaderParametersMetadataAllocations;
class UComputeDataInterface;
class UComputeDataProvider;
class UComputeKernel;
class UComputeKernelSource;

/** 
 * Description of a single edge in a UComputeGraph. 
 * todo[CF]: Consider better storage for graph data structure that is easier to interrogate efficiently.
 */
USTRUCT()
struct FComputeGraphEdge
{
	GENERATED_BODY()

	UPROPERTY()
	int32 KernelIndex;
	UPROPERTY()
	int32 KernelBindingIndex;
	UPROPERTY()
	int32 DataInterfaceIndex;
	UPROPERTY()
	int32 DataInterfaceBindingIndex;
	UPROPERTY()
	bool bKernelInput;

	// Optional name to use for the proxy generation function, in case the kernel expects
	// something other than the interface's bind name. Leave empty to go with the default. 
	UPROPERTY()
	FString BindingFunctionNameOverride;
	// Optional namespace to wrap the binding function in. A blank mean global namespace.
	UPROPERTY()
	FString BindingFunctionNamespace;

	FComputeGraphEdge()
		: KernelIndex(0)
		, KernelBindingIndex(0)
		, DataInterfaceIndex(0)
		, DataInterfaceBindingIndex(0)
		, bKernelInput(false)
	{}
};

/** 
 * Class representing a Compute Graph.
 * This holds the basic topology of the graph and is responsible for linking Kernels with Data Interfaces and compiling the resulting shader code.
 * Multiple Compute Graph asset types can derive from this to specialize the graph creation process. 
 * For example the Animation Deformer system provides a UI for creating UComputeGraph assets.
 */
UCLASS(MinimalAPI)
class UComputeGraph : public UObject
{
	GENERATED_BODY()

	friend class FComputeGraphBuilder;

protected:
	/** Kernels in the graph. */
	UPROPERTY()
	TArray<TObjectPtr<UComputeKernel>> KernelInvocations;

	/** Data interfaces in the graph. */
	UPROPERTY()
	TArray<TObjectPtr<UComputeDataInterface>> DataInterfaces;

	/** Edges in the graph between kernels and data interfaces. */
	UPROPERTY()
	TArray<FComputeGraphEdge> GraphEdges;

	/** Registered binding object class types. */
	UPROPERTY()
	TArray<TObjectPtr<UClass>> Bindings;

	/** Mapping of DataInterfaces array index to Bindings index. */
	UPROPERTY()
	TArray<int32> DataInterfaceToBinding;

public:
	COMPUTEFRAMEWORK_API UComputeGraph();
	COMPUTEFRAMEWORK_API UComputeGraph(const FObjectInitializer& ObjectInitializer);
	COMPUTEFRAMEWORK_API UComputeGraph(FVTableHelper& Helper);

	/** Called each time that a single kernel shader compilation is completed. */
	virtual void OnKernelCompilationComplete(int32 InKernelIndex, FComputeKernelCompileResults const& InCompileResults) {}

	/** 
	 * Returns true if graph is valid. 
	 * A valid graph should be guaranteed to compile, assuming the underlying shader code is well formed. 
	 */
	COMPUTEFRAMEWORK_API bool ValidateGraph(FString* OutErrors = nullptr);

	/** Get the number of binding objects used by the graph. */
	int32 GetNumBindingObjects() const { return Bindings.Num(); }

	/** 
	 * Create UComputeDataProvider objects to match the current UComputeDataInterface objects and initialize them. 
	 * We attempt to setup bindings from the InBindingObjects.
	 * The caller is responsible for any data provider binding not handled by the default behavior.
	 */
	COMPUTEFRAMEWORK_API void CreateDataProviders(int32 InBindingIndex, TObjectPtr<UObject>& InBindingObject, TArray< TObjectPtr<UComputeDataProvider> >& InOutDataProviders) const;

	/** Initialize the data providers. */
	COMPUTEFRAMEWORK_API void InitializeDataProviders(int32 InBindingIndex, TObjectPtr<UObject>& InBindingObject, TArray< TObjectPtr<UComputeDataProvider> >& InDataProviders) const;

	/** Returns true if there is a valid DataProvider entry for each of our DataInterfaces. */
	COMPUTEFRAMEWORK_API bool ValidateProviders(TArray< TObjectPtr<UComputeDataProvider> > const& DataProviders) const;

	/** Get the render proxy which is a copy of all data required by the render thread. */
	COMPUTEFRAMEWORK_API FComputeGraphRenderProxy const* GetRenderProxy() const;

	/**
	 * Call after changing the graph to build the graph resources for rendering.
	 * This will trigger any required shader compilation. By default it does async compilation
	 */
	COMPUTEFRAMEWORK_API void UpdateResources(bool bSync = false);

	/**
	 * Shader compilations triggered by UpdateResources() are async,
	 * This checks if all kernel resources are ready.
	 */	
	COMPUTEFRAMEWORK_API bool HasKernelResourcesPendingShaderCompilation() const;

protected:
	//~ Begin UObject Interface.
	COMPUTEFRAMEWORK_API void Serialize(FArchive& Ar) override;
	COMPUTEFRAMEWORK_API void PostLoad() override;
	COMPUTEFRAMEWORK_API void BeginDestroy() override;
#if WITH_EDITOR
	COMPUTEFRAMEWORK_API void OnCookEvent(UE::Cook::ECookEvent CookEvent, UE::Cook::FCookEventContext& CookContext) override;
	COMPUTEFRAMEWORK_API void BeginCacheForCookedPlatformData(ITargetPlatform const* TargetPlatform) override;
	COMPUTEFRAMEWORK_API bool IsCachedCookedPlatformDataLoaded(ITargetPlatform const* TargetPlatform) override;
	COMPUTEFRAMEWORK_API void ClearCachedCookedPlatformData(ITargetPlatform const* TargetPlatform) override;
	COMPUTEFRAMEWORK_API void ClearAllCachedCookedPlatformData() override;
#endif //WITH_EDITOR
	//~ End UObject Interface.

private:
	/** Build the shader metadata which describes bindings for a kernel with its linked data interfaces.*/
	FShaderParametersMetadata* BuildKernelShaderMetadata(int32 InKernelIndex, FShaderParametersMetadataAllocations& InOutAllocations) const;
	/** Build the shader permutation vectors for all kernels in the graph. */
	void BuildShaderPermutationVectors(TArray<FComputeKernelPermutationVector>& OutShaderPermutationVectors) const;
	/** Create the render proxy. */
	FComputeGraphRenderProxy* CreateRenderProxy() const;
	/** Release the render proxy. */
	void ReleaseRenderProxy(FComputeGraphRenderProxy* InProxy) const;
	/** Returns input and output masks for the given data interface. */
	void GetDataInterfaceInputOutputMasks(int32 InDataInterfaceIndex, uint64& OutInputMask, uint64& OutOutputMask) const;

#if WITH_EDITOR
	/** Build the HLSL source for a kernel with its linked data interfaces. */
	FString BuildKernelSource(
		int32 KernelIndex, 
		UComputeKernelSource const& InKernelSource,
		TMap<FString, FString> const& InAdditionalSources,
		FString& OutHashKey,
		TMap<FString, FString>& OutGeneratedSources,
		FComputeKernelDefinitionSet& OutDefinitionSet,
		FComputeKernelPermutationVector& OutPermutationVector) const;

	/** Cache shader resources for all kernels in the graph. */
	void CacheResourceShadersForRendering(uint32 CompilationFlags);
	
	/** For cooked packages, promote inline shader maps to the render thread. Returns false if any kernel was missing an inline shader map. */
	bool CacheCookedResourceShadersForRendering();

	/** Cache shader resources for a specific compute kernel. This will trigger any required shader compilation. */
	static void CacheShadersForResource(
		EShaderPlatform ShaderPlatform,
		ITargetPlatform const* TargetPlatform,
		uint32 CompilationFlags,
		FComputeKernelResource* Kernel);

	/** Callback to handle result of kernel shader compilations. */
	void ShaderCompileCompletionCallback(FComputeKernelResource const* KernelResource);
#endif

private:
	/** 
	 * Each kernel requires an associated FComputeKernelResource object containing the shader resources.
	 * Depending on the context (during serialization, editor, cooked game) there may me more than one object.
	 * This structure stores them all.
	 */
	struct FComputeKernelResourceSet
	{
#if WITH_EDITORONLY_DATA
		/** Kernel resource objects stored per feature level. */
		TUniquePtr<FComputeKernelResource> KernelResourcesByFeatureLevel[ERHIFeatureLevel::Num];
#else
		/** Cooked game has a single kernel resource object. */
		TUniquePtr<FComputeKernelResource> KernelResource;
#endif

#if WITH_EDITORONLY_DATA
		/** Serialized resources waiting for processing during PostLoad(). */
		TArray< TUniquePtr<FComputeKernelResource> > LoadedKernelResources;
		/** Cached resources waiting for serialization during cook. */
		TMap< const class ITargetPlatform*, TArray< TUniquePtr<FComputeKernelResource> > > CachedKernelResourcesForCooking;
#endif

		/** Release all resources. */
		void Reset();
		/** Get the appropriate kernel resource for rendering. */
		FComputeKernelResource const* Get() const;
		/** Get the appropriate kernel resource for rendering. Create a new empty resource if one doesn't exist. */
		FComputeKernelResource* GetOrCreate();
		/** Serialize the resources including the shader maps. */
		void Serialize(FArchive& Ar);
		/** Apply shader maps found in Serialize(). Call this from PostLoad(). */
		void ProcessSerializedShaderMaps();
		/** Cache inline shader map for rendering. Returns false if no inline shader map was available. */
		bool CacheInlineShaders();

#if WITH_EDITORONLY_DATA
		void CancelCompilation();
#endif
	};

	/** Kernel resources stored with the same indexing as the KernelInvocations array. */
	TArray<FComputeKernelResourceSet>  KernelResources;
	
	/** Indices of kernels pending shader compilation */
	TSet<int32> KernelResourceIndicesPendingShaderCompilation;
	
	/** Render proxy that owns all render thread resources. */
	FComputeGraphRenderProxy* RenderProxy = nullptr;
};
