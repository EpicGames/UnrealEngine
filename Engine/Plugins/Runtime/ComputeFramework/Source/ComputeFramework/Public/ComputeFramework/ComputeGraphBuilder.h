// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

class UComputeGraph;
class UComputeKernelSource;
class UComputeDataInterface;

#define UE_API COMPUTEFRAMEWORK_API

#if WITH_EDITOR

/** Opaque handle to a kernel added to FComputeGraphBuilder. */
struct FKernelHandle
{
	int32 Index = INDEX_NONE;
};

/** Opaque handle to a data interface added to FComputeGraphBuilder. */
struct FInterfaceHandle
{
	int32 Index = INDEX_NONE;
};

/**
 * Programmatic builder for UComputeGraph.
 * Handles the direct array manipulation and name-matching that are required when directly filling the UComputeGraph properties. 
 * Expected to be used as a temporary object to do the build. Passed in UObject pointers aren't GC tracked.
 */
class FComputeGraphBuilder
{
public:
	FComputeGraphBuilder() = default;
	FComputeGraphBuilder(const FComputeGraphBuilder&) = delete;
	FComputeGraphBuilder& operator=(const FComputeGraphBuilder&) = delete;

	/** 
	 * Add a kernel. Source must outlive the builder (typically owned by a UObject outer). 
	 * Source will be modified by Build() which resets it's inputs and outputs to the new ones applied by the builder.
	 */
	UE_API FKernelHandle AddKernel(UComputeKernelSource* Source, FName Name = NAME_None);

	/** Add a data interface. BindingClass is the UObject subclass that will be bound at runtime. */
	UE_API FInterfaceHandle AddDataInterface(UComputeDataInterface* Interface, UClass* BindingClass, FName Name = NAME_None);

	/** Connect a kernel HLSL input parameter to a data interface read function. */
	UE_API void ConnectInput(FKernelHandle Kernel, FStringView KernelFn, FInterfaceHandle Interface, FStringView InterfaceFn);

	/** Connect a kernel HLSL output parameter to a data interface write function. */
	UE_API void ConnectOutput(FKernelHandle Kernel, FStringView KernelFn, FInterfaceHandle Interface, FStringView InterfaceFn);

	/**
	 * Apply the built topology to OutGraph, resetting any existing topology first.
	 * Outer is used as the UObject outer for the UComputeKernel wrapper objects created internally.
	 */
	UE_API void Build(UComputeGraph& OutGraph, UObject& Outer) const;

private:
	struct FPinEntry
	{
		FString KernelFunctionName;
		int32 InterfaceIndex = INDEX_NONE;
		FString InterfaceFunctionName;
	};

	struct FKernelEntry
	{
		UComputeKernelSource* Source;
		FName Name;
		TArray<FPinEntry> Inputs;
		TArray<FPinEntry> Outputs;
	};

	struct FInterfaceEntry
	{
		UComputeDataInterface* Interface;
		UClass* BindingClass;
		FName Name;
	};

	TArray<FKernelEntry> KernelEntries;
	TArray<FInterfaceEntry> InterfaceEntries;
};

#endif // WITH_EDITOR

#undef UE_API
