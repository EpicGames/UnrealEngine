// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Components/ActorComponent.h"
#include "ComputeFramework/ComputeGraphInstance.h"
#include "ComputeGraphComponent.generated.h"

#define UE_API COMPUTEFRAMEWORK_API

/** 
 * Component which holds a context for a UComputeGraph.
 * This object creates the graph data providers, and queues the execution. 
 */
UCLASS(MinimalAPI, Blueprintable, Category = ComputeFramework, meta = (BlueprintSpawnableComponent))
class UComputeGraphComponent : public UActorComponent
{
	GENERATED_BODY()

public:
	UE_API UComputeGraphComponent();

	/** Initialize a Data Provider object for the ComputeGraph. */
	UFUNCTION(BlueprintNativeEvent, Category = "Compute")
	UE_API void InitializeProvider(int32 InDataInterfaceIndex, UObject* InOutDataProvider);

	/** Queue the graph for execution at the next render update. */
	UFUNCTION(BlueprintCallable, Category = "Compute")
	UE_API void QueueExecute();

protected:
	/** The Compute Graph asset. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Compute")
	TObjectPtr<UComputeGraph> ComputeGraph = nullptr;

	/** Get the binding object for the ComputeGraph. */
	UFUNCTION(BlueprintNativeEvent, Category = "Compute")
	UObject* GetBindingObject(int32 InBindingIndex);

	/** Serialized data provider properties. */
	UPROPERTY(EditAnywhere, Category = "Compute", Instanced, NoClear, meta = (ShowOnlyInnerProperties))
	TArray<TObjectPtr<UComputeDataProvider>> DataProviderTemplates;

	/** The Instance with associated per instance settings. */
	UPROPERTY()
	FComputeGraphInstance ComputeGraphInstance;

	//~ Begin UActorComponent Interface
	void OnRegister() override;
	void OnUnregister() override;
	void SendRenderDynamicData_Concurrent() override;
	void DestroyRenderState_Concurrent() override;
	bool ShouldCreateRenderState() const override { return true; }
	//~ End UActorComponent Interface
};

#undef UE_API
