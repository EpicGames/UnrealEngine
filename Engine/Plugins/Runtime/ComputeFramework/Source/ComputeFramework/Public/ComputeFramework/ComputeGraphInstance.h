// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Async/Future.h"
#include "ComputeFramework/ComputeFrameworkKernelTiming.h"
#include "RHIDefinitions.h"
#include "ComputeGraphInstance.generated.h"

#define UE_API COMPUTEFRAMEWORK_API

class FSceneInterface;
class UComputeDataProvider;
class UComputeGraph;

/** 
 * Class to store a set of data provider bindings for UComputeGraph and to
 * enqueue work to the ComputeFramework's compute system.
 */
USTRUCT()
struct FComputeGraphInstance
{
	GENERATED_USTRUCT_BODY();

public:
	/** 
	 * Set the priority used when sorting work. 
	 * Kernels in instances with a lower sort prioirty will always be submitted first.
	 */
	void SetGraphSortPriority(uint8 InPriority) { GraphSortPriority = InPriority; }

	/** 
	 * Create and initialize the Data Provider objects for a single binding of a ComputeGraph. 
	 * The type of binding object is expected to match the associated Binding on the UComputeGraph.
	 */
	UE_API void CreateDataProviders(UComputeGraph* InComputeGraph, int32 InBindingIndex, TObjectPtr<UObject> InBindingObject);

	/**
	 * Initialize the Data Provider objects for a single binding of a ComputeGraph.
	 * The type of binding object is expected to match the associated Binding on the UComputeGraph.
	 */
	UE_API void InitializeDataProviders(UComputeGraph* InComputeGraph, int32 InBindingIndex, TObjectPtr<UObject> InBindingObject);

	/** Clear the state within the Data Provider objects. */
	UE_API void ResetDataProviders();

	/** Create the Data Provider objects. */
	UE_API void DestroyDataProviders();

	/** Get the Data Provider objects. */
	TArray< TObjectPtr<UComputeDataProvider> >& GetDataProviders() { return DataProviders; }

	/** Get the number of Data Provider objects. */
	int GetNumDataProviders() const { return DataProviders.Num(); }

	/** Enqueue the ComputeGraph work. */
	UE_API bool EnqueueWork(
		UComputeGraph* InComputeGraph,
		FSceneInterface const* InScene,
		FName InExecutionGroupName,
		FName InOwnerName,
		FSimpleDelegate InFallbackDelegate,
		UObject* InOwnerPointer = nullptr,
		uint8 InGraphSortPriorityOffset = 0
#if HAS_GPU_STATS
		, TArray<TPromise<FComputeFrameworkKernelTiming>> InKernelTimingPromises = {}
#endif
	);

	void SetRenderCapturesEnabled(bool bInEnable) { bEnableRenderCaptures = bInEnable; }

private:
	/** The currently bound Data Provider objects. */
	UPROPERTY(Transient)
	TArray< TObjectPtr<UComputeDataProvider> > DataProviders;

	/** Priority used when sorting work. */
	uint8 GraphSortPriority = 0;

	bool bEnableRenderCaptures = false;
};

#undef UE_API
