// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "ComputeKernel.generated.h"

#define UE_API COMPUTEFRAMEWORK_API

class UComputeKernelSource;

/** Flags that convey kernel behavior to aid compilation/optimizations. */
UENUM(meta = (Bitflags, UseEnumValuesAsMaskValuesInEditor = "true"))
enum class EComputeKernelFlags : uint32
{
	/*
	 * Default implies that this kernel must be compiled before the system is functional.
	 * It also implies that this will be compiled synchronously. Other than a pass
	 * through kernel, default shouldn't be used.
	 */
	IsDefaultKernel = 1 << 0,

	/*
	 * Promise from the author that all memory writes will be unique per shader
	 * dispatch thread. i.e. ThreadX will be the only thread to write to MemoryY,
	 * thus no synchronization is necessary by the compute graph.
	 */
	IsolatedMemoryWrites = 1 << 1,

	/**
	 * Trigger render capture when this kernel dispatches, for debugging/testing purposes.
	 */
	TriggerRenderCapture = 1 << 2,
};
ENUM_CLASS_FLAGS(EComputeKernelFlags);

/** Base class representing a kernel that will be run as a shader on the GPU. */
UCLASS(MinimalAPI, hidecategories = (Object))
class UComputeKernel : public UObject
{
	GENERATED_BODY()

public:
	/** 
	 * The compute kernel source asset.
	 * A kernel's source may be authored by different mechanisms; e.g. HLSL text, VPL graph, ML Meta Lang, etc
	 */
	UPROPERTY(EditAnywhere, AssetRegistrySearchable, meta = (ShowOnlyInnerProperties), Category = "Kernel")
	TObjectPtr<UComputeKernelSource> KernelSource = nullptr;

	/** Specifying certain memory access flags allows for optimizations such as kernel fusing. */
	UPROPERTY(VisibleAnywhere, BlueprintReadOnly, meta = (Bitmask, BitmaskEnum = "/Script/ComputeFramework.EComputeKernelFlags"), Category = "Kernel")
	int32 KernelFlags = 0;

protected:
#if WITH_EDITOR
	//~ Begin UObject Interface.
	UE_API void PostLoad() override;
	//~ End UObject Interface.
#endif
};

#undef UE_API
