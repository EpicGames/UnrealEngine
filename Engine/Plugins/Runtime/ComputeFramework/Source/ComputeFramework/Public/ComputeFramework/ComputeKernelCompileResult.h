// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

/** Struct containing a message returned by shader compilation. */
struct FComputeKernelCompileMessage
{
	enum class EMessageType
	{
		None,
		Info,
		Warning,
		Error
	};

	EMessageType Type = EMessageType::None;
	FString Text;
	FString VirtualFilePath;
	FString RealFilePath;
	int32 Line = -1;
	int32 ColumnStart = -1;
	int32 ColumnEnd = -1;
	
	/** Equality operator used to deduplicate messages. */
	bool operator==(FComputeKernelCompileMessage const& Rhs) const
	{
		return Type == Rhs.Type && Text == Rhs.Text && VirtualFilePath == Rhs.VirtualFilePath && RealFilePath == Rhs.RealFilePath && Line == Rhs.Line && ColumnStart == Rhs.ColumnStart && ColumnEnd == Rhs.ColumnEnd;
	}
};

/** Struct containing all of the messages returned by shader compilation. */
struct FComputeKernelCompileResults
{
	TArray<FComputeKernelCompileMessage> Messages;
};
