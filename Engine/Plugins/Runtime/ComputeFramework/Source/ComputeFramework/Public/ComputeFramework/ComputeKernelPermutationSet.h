// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ComputeKernelPermutationSet.generated.h"

USTRUCT()
struct FComputeKernelPermutationBool
{
	GENERATED_BODY()

	UPROPERTY(VisibleAnywhere, Category = "Permutation Options")
	FString Name;

	UPROPERTY(EditDefaultsOnly, Category = "Permutation Options")
	bool Value = false;

	FComputeKernelPermutationBool() = default;

	explicit FComputeKernelPermutationBool(FString InName, bool InValue)
		: Name(MoveTemp(InName))
		, Value(InValue)
	{
	}
};

USTRUCT()
struct FComputeKernelPermutationSet
{
	GENERATED_BODY()

	UPROPERTY(EditDefaultsOnly, EditFixedSize, meta = (EditFixedOrder), Category = "Permutation Options")
	TArray<FComputeKernelPermutationBool> BooleanOptions;
};

USTRUCT()
struct FComputeKernelDefinition
{
	GENERATED_BODY()

public:
	UPROPERTY(VisibleAnywhere, Category = "Kernel")
	FString Symbol;

	UPROPERTY(EditDefaultsOnly, Category = "Kernel")
	FString Define;

	FComputeKernelDefinition() = default;

	explicit FComputeKernelDefinition(FString InSymbol, FString InDefine = FString())
		: Symbol(MoveTemp(InSymbol))
		, Define(MoveTemp(InDefine))
	{
	}
};

USTRUCT()
struct FComputeKernelDefinitionSet
{
	GENERATED_BODY()

public:
	UPROPERTY(EditDefaultsOnly, EditFixedSize, meta = (EditFixedOrder), Category = "Permutation Options")
	TArray<FComputeKernelDefinition> Defines;
};
