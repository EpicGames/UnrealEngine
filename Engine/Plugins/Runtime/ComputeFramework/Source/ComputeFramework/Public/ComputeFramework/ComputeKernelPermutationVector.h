// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "ComputeKernelPermutationVector.generated.h"

#define UE_API COMPUTEFRAMEWORK_API

USTRUCT()
struct FComputeKernelPermutationVector
{
	GENERATED_BODY()

	union FPermutationBits
	{
		uint32 PackedValue = 0;
	
		struct
		{
			uint16 BitIndex;
			uint16 NumValues;
		};
	};

	/** Map from permutation define name to packed FPermutaionBits value. */
	UPROPERTY()
	TMap<FString, uint32> Permutations;
	
	/** Number of permutation bits allocated. */
	UPROPERTY()
	uint32 BitCount = 0;

	/** Add a permutation to the vector. */
	UE_API void AddPermutation(FString const& Name, uint32 NumValues);

	/** Add a permutation to the vector. */
	UE_API void AddPermutationSet(struct FComputeKernelPermutationSet const& PermutationSet);

	/** Get the permutation bits for a given Name and Value. */
	UE_API uint32 GetPermutationBits(FString const& Name, uint32 Value) const;
	UE_API uint32 GetPermutationBits(FString const& Name, uint32 PrecomputedNameHash, uint32 Value) const;
};

#undef UE_API
