// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ComputeFramework/ComputeKernelPermutationSet.h"
#include "ComputeFramework/ShaderParamTypeDefinition.h"
#include "ComputeKernelSource.generated.h"

#define UE_API COMPUTEFRAMEWORK_API

class UComputeSource;

/** 
 * Class representing the source for a UComputeKernel 
 * We derive from this for each authoring mechanism. (HLSL text, VPL graph, ML Meta Lang, etc.)
 */
UCLASS(MinimalAPI, Abstract)
class UComputeKernelSource : public UObject
{
	GENERATED_BODY()

public:
#if WITH_EDITOR
	/** Get kernel source code ready for HLSL compilation. */
	virtual FString GetSource() const PURE_VIRTUAL(UComputeKernelSource::GetSource, return {};);
#endif

	/** Kernel entry point. */
	UPROPERTY(VisibleAnywhere, AssetRegistrySearchable, Category = "Kernel")
	FString EntryPoint;

	/** Kernel group size. */
	UPROPERTY(VisibleAnywhere, AssetRegistrySearchable, Category = "Kernel")
	FIntVector GroupSize = FIntVector(64, 1, 1);

#if WITH_EDITORONLY_DATA
	/** Base permutations exposed by the kernel. These will be extended by further permutations declared in any linked data providers. */
	UPROPERTY(VisibleAnywhere, meta = (ShowOnlyInnerProperties), Category = "Kernel")
	FComputeKernelPermutationSet PermutationSet;

	/** Base environment defines for kernel compilation. These will be extended by further defines declared in any linked data providers. */
	UPROPERTY(VisibleAnywhere, meta = (ShowOnlyInnerProperties), Category = "Kernel")
	FComputeKernelDefinitionSet DefinitionsSet;

	/** An array of additional independent source assets that the kernel source depends on. */
	UPROPERTY(EditAnywhere, meta = (ShowOnlyInnerProperties), Category = "External")
	TArray<TObjectPtr<UComputeSource>> AdditionalSources;

	/* Named external inputs for the kernel. These must be fulfilled by linked data providers. */
	UPROPERTY(VisibleAnywhere, EditFixedSize, Category = "External")
	TArray<FShaderFunctionDefinition> ExternalInputs;

	/* Named external outputs for the kernel. These must be fulfilled by linked data providers. */
	UPROPERTY(VisibleAnywhere, EditFixedSize, Category = "External")
	TArray<FShaderFunctionDefinition> ExternalOutputs;
#endif // WITH_EDITORONLY_DATA
};

/** Simple concrete implementation of UComputeKernelSource with contained source text property. */
UCLASS(MinimalAPI)
class UComputeKernelSourceWithText final : public UComputeKernelSource
{
	GENERATED_BODY()

public:
#if WITH_EDITORONLY_DATA
	/* Kernel source text. */
	UPROPERTY(EditAnywhere, Category = "Kernel")
	FString SourceText;
#endif

#if WITH_EDITOR
	//~ Begin UComputeKernelSource Interface.
	UE_API virtual FString GetSource() const override;
	//~ End UComputeKernelSource Interface.
#endif
};

#undef UE_API
