// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#define UE_API COMPUTEFRAMEWORK_API

struct FShaderValueTypeHandle;
class FShaderParametersMetadataBuilder;
class FShaderParametersMetadata;

namespace ComputeFramework
{
	COMPUTEFRAMEWORK_API void AddParamForType(FShaderParametersMetadataBuilder& InOutBuilder, TCHAR const* InName, FShaderValueTypeHandle const& InValueType, TArray<FShaderParametersMetadata*>& OutNestedStructs);

	/** Helper struct to convert shader value type to shader parameter metadata */
	 struct FTypeMetaData
	{
		UE_API FTypeMetaData(FShaderValueTypeHandle InType);
		FTypeMetaData(const FTypeMetaData& InOther) = delete;
		FTypeMetaData& operator=(const FTypeMetaData& InOther) = delete;
		UE_API ~FTypeMetaData();

		const FShaderParametersMetadata* Metadata;
		
		TArray<FShaderParametersMetadata*> AllocatedMetadatas;
	};	
}

#undef UE_API
