// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "ComputeSource.generated.h"

/** 
 * Class representing some source for inclusion in a UComputeKernel.
 * We derive from this for each authoring mechanism. (HLSL text, VPL graph, ML Meta Lang, etc.)
 */
UCLASS(MinimalAPI, Abstract)
class UComputeSource : public UObject
{
	GENERATED_BODY()

public:
	/** Get source code ready for HLSL compilation. */
	virtual FString GetSource() const { return FString(); }
	/** Get virtual file path for the source code. This will be the file name shown in any compilation errors. */
	virtual FString GetVirtualPath() const { return FString::Printf(TEXT("/Engine/Generated/%s.ush"), *GetPathName()); }

public:
	/** Array of additional source objects. This allows us to specify source dependencies. */
	UPROPERTY(EditAnywhere, Category = Source)
	TArray<TObjectPtr<UComputeSource>> AdditionalSources;
};
