// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/NameTypes.h"
#include "ShaderParamTypeDefinition.generated.h"

#define UE_API COMPUTEFRAMEWORK_API

class FArchive;
struct FShaderValueType;
struct FShaderParamTypeDefinition;

/* The base types of data that shaders can consume/expose */
UENUM()
enum class EShaderFundamentalType : uint8
{
	Bool	= 0,
	Int		= 1,
	Uint	= 2,
	Float	= 3,
	Struct	= 4,
	None	= 255u,
};

/*
 * Shader types can then be in the form of a scalar, vector, matrix.
 * e.g Scalar: float a; 	Vector: float3 n; 		Matrix: float3x4 WVP;
 * Note: float b[5]; is still considered scalar. It is an array of scalars.
 */
UENUM()
enum class EShaderFundamentalDimensionType : uint8
{
	Scalar,
	Vector,
	Matrix,
};

/* Describes how the shader parameters are bound. */
UENUM()
enum class EShaderParamBindingType : uint8
{
	None,
	ConstantParameter,
	ReadOnlyResource,  // SRV, treated as Inputs
	ReadWriteResource, // UAV, treated as Outputs
};

/*  */
UENUM()
enum class EShaderResourceType : uint8
{
	None,
	Texture1D,
	Texture2D,
	Texture3D,
	TextureCube,
	Buffer,
	StructuredBuffer,
	ByteAddressBuffer,
};

UENUM()
enum class EShaderParamModifier : uint8
{
	None,
	In,
	Out,
	InOut,
};

/*  */
USTRUCT()
struct FShaderValueTypeHandle
{
	GENERATED_BODY()

	const FShaderValueType *ValueTypePtr = nullptr;

	bool IsValid() const
	{
		return ValueTypePtr != nullptr;
	}

	explicit operator bool() const
	{
		return IsValid();
	}

	const FShaderValueType *operator->() const
	{
		return ValueTypePtr;
	}

	const FShaderValueType &operator*() const
	{
		return *ValueTypePtr;
	}

	bool operator==(const FShaderValueTypeHandle &InOther) const
	{
		return ValueTypePtr == InOther.ValueTypePtr;
	}
	
	bool operator!=(const FShaderValueTypeHandle &InOther) const
	{
		return ValueTypePtr != InOther.ValueTypePtr;
	}
	
	bool Serialize(FArchive& Ar);
};

FArchive& operator<<(FArchive& InArchive, FShaderValueTypeHandle& InHandle);

template<> struct TStructOpsTypeTraits<FShaderValueTypeHandle> : TStructOpsTypeTraitsBase2<FShaderValueTypeHandle>
{
	enum { WithSerializer = true, WithIdenticalViaEquality = true, WithCopy = false };
};



USTRUCT()
struct FArrayShaderValue
{
	GENERATED_BODY()

	UPROPERTY()
	TArray<uint8> ArrayOfValues;
};

USTRUCT()
struct FShaderValueContainer
{
	GENERATED_BODY()

	UPROPERTY()
	TArray<uint8> ShaderValue;

	UPROPERTY()
	TArray<FArrayShaderValue> ArrayList;

	FShaderValueContainer() = default;
	
	explicit FShaderValueContainer(int32 InShaderValueSize, int32 InNumArrays)
	{
		ShaderValue.SetNumUninitialized(InShaderValueSize);
		ArrayList.AddDefaulted(InNumArrays);
	}

	static bool IsSameType(const FShaderValueContainer& A, const FShaderValueContainer& B)
	{
		return A.ShaderValue.Num() == B.ShaderValue.Num() && A.ArrayList.Num() == B.ArrayList.Num();
	};

	bool IsValid() const
	{
		return ShaderValue.Num() > 0 || ArrayList.Num() > 0;
	}
	void Reset()
	{
		ShaderValue.Reset();
		ArrayList.Reset();
	};
};

struct FShaderValueContainerView
{
	TArrayView<uint8> ShaderValue;
	TArrayView<FArrayShaderValue> ArrayList;

	FShaderValueContainerView(
			TArrayView<uint8> InShaderValue,
			TArrayView<FArrayShaderValue> InArrayList) :
			ShaderValue(InShaderValue),
			ArrayList(InArrayList) {}
		
	FShaderValueContainerView(TArrayView<uint8> InShaderValue) :
		ShaderValue(InShaderValue) {}
		
	FShaderValueContainerView(FShaderValueContainer& InValue) :
		ShaderValue(InValue.ShaderValue),
		ArrayList(InValue.ArrayList) {}
};

/*  */
USTRUCT()
struct FShaderValueType
{
	GENERATED_BODY()
	
	// A simple container representing a single, named element in a shader value struct.
	struct FStructElement
	{
		FStructElement() = default;
		FStructElement(FName InName, FShaderValueTypeHandle InType)
		    : Name(InName), Type(InType) {}

		bool operator==(const FStructElement &InOther) const
		{
			return Name == InOther.Name && Type.ValueTypePtr == InOther.Type.ValueTypePtr;
		}

		bool operator!=(const FStructElement& InOther) const { return !(*this == InOther); }

		friend FArchive& operator<<(FArchive& InArchive, FStructElement& InElement);

		FName Name;
		FShaderValueTypeHandle Type;
	};

	/** Returns a scalar value type. If the fundamental type given is invalid for scalar values 
	  * (e.g. struct), then this function returns a nullptr. 
	  */
	static UE_API FShaderValueTypeHandle Get(EShaderFundamentalType InType);

	/** Returns a vector value type. InElemCount can be any value between 1-4. If the type 
	  * given is invalid for scalar values (e.g. struct) or InElemCount is out of range, then 
	  * this function returns a nullptr. 
	  */
	static UE_API FShaderValueTypeHandle Get(EShaderFundamentalType InType, int32 InElemCount);

	/** Constructor for vector values. InElemCount can be any value between 1-4 */
	static UE_API FShaderValueTypeHandle Get(EShaderFundamentalType InType, int32 InRowCount, int32 InColumnCount);

	/** Constructor for struct types */
	static UE_API FShaderValueTypeHandle Get(FName InName, std::initializer_list<FStructElement> InStructElements);

	/** Constructor for struct types */
	static UE_API FShaderValueTypeHandle Get(FName InName, const TArray<FStructElement>& InStructElements);
	
	/** Construct an array type from an existing type */
	static UE_API FShaderValueTypeHandle MakeDynamicArrayType(const FShaderValueTypeHandle& InElementType);
	
	/** Parses the given string section and tries to convert to a shader value type.
	 *  NOTE: Does not work on structs.
	 */
	static UE_API FShaderValueTypeHandle FromString(const FString& InTypeDecl);
	
	/** Returns true if this type and the other type are exactly equal. */
	UE_API bool operator==(const FShaderValueType &InOtherType) const;

	/** Returns true if this type and the other type are not equal. */
	bool operator!=(const FShaderValueType& InOtherType) const
	{
		return !(*this == InOtherType);
	}

	friend FArchive& operator<<(FArchive& InArchive, FShaderValueTypeHandle& InHandle);

	/** Returns the type name as a string (e.g. 'vector2', 'matrix2x3' or 'struct_name') for 
	    use in variable declarations. */
	UE_API FString ToString(const FName& InStructTypeNameOverride = {}) const;

	/** Returns the type declaration if this type is a struct, or the empty string if not. */
	UE_API FString GetTypeDeclaration(const TMap<FName, FName>& InNamesToReplace = {}, bool bCommentPaddings = false) const;

	/** Returns the all the struct types used in this type if the type is a struct. */
	UE_API TArray<FShaderValueTypeHandle> GetMemberStructTypes() const;

	/** Returns the size in bytes required to hold one element of this type using HLSL sizing
	  * (which may be different from packed sizing in C++).
	  */
	UE_API int32 GetResourceElementSize() const;

	/** Returns a zero value for the type as a string, suitable for use as a constant in HLSL
	  * code.
	  */   
	UE_API FString GetZeroValueAsString() const;
	
	UPROPERTY()
	EShaderFundamentalType Type = EShaderFundamentalType::Bool;

	UPROPERTY()
	EShaderFundamentalDimensionType DimensionType = EShaderFundamentalDimensionType::Scalar;

	union
	{
		uint8 VectorElemCount;

		struct
		{
			uint8 MatrixRowCount;
			uint8 MatrixColumnCount;
		};
	};

	UPROPERTY()
	FName Name;

	UPROPERTY()
	bool bIsDynamicArray = false;
	
	TArray<FStructElement> StructElements;

private:
	static UE_API FShaderValueTypeHandle GetOrCreate(FShaderValueType &&InValueType);
};

/**
* A hashing function to allow the FShaderValueType class to be used with hashing containers (e.g.
* TSet or TMap).
*/
COMPUTEFRAMEWORK_API uint32 GetTypeHash(const FShaderValueType& InShaderValueType);

/* Fully describes the name and type of a parameter a shader exposes. */
USTRUCT()
struct FShaderParamTypeDefinition
{
	GENERATED_BODY()

public:
	static UE_API EShaderFundamentalType ParseFundamental(
		const FString& Str
		);

	static UE_API EShaderFundamentalDimensionType ParseDimension(
		const FString& Str
		);

	static UE_API uint8 ParseVectorDimension(
		const FString& Str
		);

	static UE_API FIntVector2 ParseMatrixDimension(
		const FString& Str
		);

	static UE_API EShaderResourceType ParseResource(
		const FString& Str
		);

public:
	UPROPERTY(VisibleAnywhere, Category = "Kernel")
	FString TypeDeclaration;

	UPROPERTY(EditAnywhere, Category = "Kernel")
	FString	Name;

	// The value type for this definition.
	UPROPERTY()
	FShaderValueTypeHandle ValueType;

	UPROPERTY()
	uint16 ArrayElementCount = 0; // 0 indicates not an array. >= 1 indicates an array

	UPROPERTY()
	EShaderParamBindingType BindingType = EShaderParamBindingType::None;

	UPROPERTY()
	EShaderResourceType	ResourceType = EShaderResourceType::None;

	UPROPERTY()
	EShaderParamModifier Modifier = EShaderParamModifier::None;
	
	bool IsAnyBufferType() const
	{
		return
			ResourceType == EShaderResourceType::Buffer ||
			ResourceType == EShaderResourceType::ByteAddressBuffer ||
			ResourceType == EShaderResourceType::StructuredBuffer;
	}

	bool IsAnyTextureType() const
	{
		return
			ResourceType == EShaderResourceType::Texture1D ||
			ResourceType == EShaderResourceType::Texture2D ||
			ResourceType == EShaderResourceType::Texture3D ||
			ResourceType == EShaderResourceType::TextureCube;
	}

	/* Determines if the type definition is valid according to HLSL rules. */
	bool IsValid() const
	{
		if (!ValueType)
		{
			return false;
		}
		
		if (ValueType->Type == EShaderFundamentalType::Struct && ValueType->DimensionType != EShaderFundamentalDimensionType::Scalar)
		{
			// cannot have anything but scalar struct types
			return false;
		}

		if (IsAnyTextureType() && ValueType->Type == EShaderFundamentalType::Struct)
		{
			// cannot have textures of structs
			return false;
		}

		if (IsAnyTextureType() && ValueType->DimensionType == EShaderFundamentalDimensionType::Matrix)
		{
			// cannot have textures of matrices
			return false;
		}

		if ((IsAnyBufferType() || IsAnyTextureType()) && BindingType == EShaderParamBindingType::ConstantParameter)
		{
			// cannot have buffers and textures bound as const params
			return false;
		}

		return true;
	}

	UE_API void ResetTypeDeclaration(
		);
};


/* Describes a shader function signature. */
USTRUCT()
struct FShaderFunctionDefinition
{
	GENERATED_BODY()

public:
	/** Function name. */
	UPROPERTY(VisibleAnywhere, Category = "Kernel")
	FString	Name;

	/** Function parameter types. */
	UPROPERTY(VisibleAnywhere, Category = "Kernel")
	TArray<FShaderParamTypeDefinition> ParamTypes;

	/** If set to true then the first element of ParamTypes is the return type. Otherwise return type is void. */
	UPROPERTY()
	bool bHasReturnType = false;

public:
	UE_API FShaderFunctionDefinition& SetName(FString InName);
	UE_API FShaderFunctionDefinition& AddParam(FShaderValueTypeHandle InValueType, EShaderParamModifier InModifier = EShaderParamModifier::None);
	UE_API FShaderFunctionDefinition& AddParam(EShaderFundamentalType InType, int32 InRowCount = 0, int32 InColumnCount = 0, EShaderParamModifier InModifier = EShaderParamModifier::None);
	UE_API FShaderFunctionDefinition& AddReturnType(FShaderValueTypeHandle InValueType);
	UE_API FShaderFunctionDefinition& AddReturnType(EShaderFundamentalType InType, int32 InRowCount = 0, int32 InColumnCount = 0);
};

#undef UE_API
