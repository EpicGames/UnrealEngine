// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Containers/ChunkedArray.h"

#define UE_API COMPUTEFRAMEWORK_API

class FShaderParametersMetadata;

/** 
 * When we build FShaderParametersMetadata at runtime it can require that we allocate
 * additional data referenced but not owned by the FShaderParametersMetadata.
 * This structure tracks the allocations, and should be released only after we are
 * done with the associated FShaderParametersMetadata.
 */
struct FShaderParametersMetadataAllocations
{
	/** Allocated metadata. Should include the parent metadata allocation. */
	TArray<FShaderParametersMetadata*> ShaderParameterMetadatas;
	/** Allocated name dictionary. */
	TChunkedArray<FString, 512> Names;

	FShaderParametersMetadataAllocations() = default;
	FShaderParametersMetadataAllocations(FShaderParametersMetadataAllocations& Other) = delete;
	UE_API ~FShaderParametersMetadataAllocations();
};

#undef UE_API
