// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
    public class ComputeFrameworkEditor : ModuleRules
    {
        public ComputeFrameworkEditor(ReadOnlyTargetRules Target) : base(Target)
        {
            PrivateDependencyModuleNames.AddRange(
                new string[]
                {
					"ComputeFramework",
					"Core",
					"Engine",
					"UnrealEd",
				}
			);
        }
    }
}
