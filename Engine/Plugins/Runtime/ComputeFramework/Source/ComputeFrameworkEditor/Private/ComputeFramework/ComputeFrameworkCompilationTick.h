// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "TickableEditorObject.h"

/** Editor tickable object used to tick the shader compilation process. */
class FComputeFrameworkCompilationTick : public FTickableEditorObject
{
public:
	ETickableTickType GetTickableTickType() const override
	{
		return ETickableTickType::Always;
	}
	
	TStatId GetStatId() const override
	{
		RETURN_QUICK_DECLARE_CYCLE_STAT(FComputeFrameworkCompilationTick, STATGROUP_Tickables);
	}
	
	void Tick(float DeltaSeconds) override;
};
