// Copyright Epic Games, Inc. All Rights Reserved.

#include "ComputeFramework/ComputeFrameworkCompilationTick.h"

#include "ComputeFramework/ComputeFramework.h"

void FComputeFrameworkCompilationTick::Tick(float DeltaSeconds)
{
	ComputeFramework::TickCompilation(DeltaSeconds);
}
