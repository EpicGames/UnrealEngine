// Copyright Epic Games, Inc. All Rights Reserved.

#include "ComputeFramework/ComputeFrameworkEditorModule.h"

#include "ComputeFramework/ComputeFrameworkCompilationTick.h"
#include "Modules/ModuleManager.h"

void FComputeFrameworkEditorModule::StartupModule()
{
	TickObject = MakeUnique<FComputeFrameworkCompilationTick>();
}

void FComputeFrameworkEditorModule::ShutdownModule()
{
	TickObject = nullptr;
}

IMPLEMENT_MODULE(FComputeFrameworkEditorModule, ComputeFrameworkEditor)
