// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class EditableComputeGraph : ModuleRules
	{
		public EditableComputeGraph(ReadOnlyTargetRules Target) : base(Target)
		{
			PublicDependencyModuleNames.AddRange(
				new string[]
				{
					"Core",
					"CoreUObject",
					"Engine",
					"ComputeFramework",
				}
			);

			PrivateDependencyModuleNames.AddRange(
				new string[]
				{
					"ComputeDataInterface",
					"RenderCore",
					"RHI",
				}
			);
		}
	}
}
