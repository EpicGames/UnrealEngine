// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ComputeFramework/ComputeGraph.h"
#include "EditableComputeGraph.generated.h"

struct FComputeKernelCompileMessage;
struct FComputeKernelCompileResults;

#define UE_API EDITABLECOMPUTEGRAPH_API

DECLARE_MULTICAST_DELEGATE_OneParam(FOnComputeGraphCompileOutput, TArray<FComputeKernelCompileMessage> const&);

/** Describes one kernel pin: a binding between a kernel HLSL parameter and a data interface function. */
USTRUCT()
struct FKernelPin
{
	GENERATED_BODY();

	/** Name of the corresponding parameter in the kernel HLSL. */
	UPROPERTY(EditAnywhere, Category = "Pin", meta = (DisplayName = "Kernel Function"))
	FString KernelFunctionName;
	/** Name of a DataInterface entry in the owning FComputeGraphDesc. */
	UPROPERTY(EditAnywhere, Category = "Pin", meta = (DisplayName = "Data Interface"))
	FName DataInterfaceName;
	/** Name of the function exposed by that data interface. */
	UPROPERTY(EditAnywhere, Category = "Pin", meta = (DisplayName = "Data Interface Function"))
	FString DataInterfaceFunctionName;
	
	/** Transient value that the details customization uses this to display an orphan warning. */
	UPROPERTY(Transient, VisibleAnywhere, Category = "Pin")
	bool bOrphaned = false;
};

/** Describes a single compute kernel: shader source and associated properties. */
USTRUCT()
struct FComputeGraphKernelDesc
{
	GENERATED_BODY();

	/** Unique name identifying this kernel within the graph. */
	UPROPERTY(EditAnywhere, Category = "Kernel")
	FName Name;
	/** Full HLSL source text for this kernel. */
	UPROPERTY(EditAnywhere, Category = "Kernel")
	FString SourceText;
	/** Name of the HLSL function that serves as the compute shader entry point. */
	UPROPERTY(EditAnywhere, Category = "Kernel")
	FString EntryPoint;
	/** Thread-group dimensions passed to the shader compiler (X * Y * Z threads per group). */
	UPROPERTY(EditAnywhere, Category = "Kernel")
	FIntVector GroupSize = FIntVector(64, 1, 1);
	/** Kernel inputs. Each entry maps a kernel HLSL parameter to a data interface read function. */
	UPROPERTY(EditAnywhere, Category = "Kernel")
	TArray<FKernelPin> Inputs;
	/** Kernel outputs. Each entry maps a kernel HLSL parameter to a data interface write function. */
	UPROPERTY(EditAnywhere, Category = "Kernel")
	TArray<FKernelPin> Outputs;
};

/** Describes a binding object: a named ActorComponent class that data interfaces can reference. */
USTRUCT()
struct FComputeGraphDataBindingObjectDesc
{
	GENERATED_BODY();

	/** Unique name identifying this binding object within the graph. */
	UPROPERTY(EditAnywhere, Category = "BindingObject")
	FName Name;
	/** The ActorComponent subclass that this binding object wraps. */
	UPROPERTY(EditAnywhere, Category = "BindingObject", meta = (AllowedClasses = "/Script/Engine.ActorComponent"))
	TObjectPtr<UClass> Type;
};

/** Describes a data interface: a named UComputeDataInterface class optionally bound to a binding object. */
USTRUCT()
struct FComputeGraphDataInterfaceDesc
{
	GENERATED_BODY();

	/** Unique name identifying this data interface within the graph. */
	UPROPERTY(EditAnywhere, Category = "DataInterface")
	FName Name;
	/** Name of the FComputeGraphDataBindingObjectDesc that provides the component instance at runtime. */
	UPROPERTY(EditAnywhere, Category = "DataInterface", meta = (DisplayName = "Binding Object"))
	FName BindingObjectName;
	/** The ComputeDataInterface subclass that implements the read/write functions for this interface. */
	UPROPERTY(EditAnywhere, Category = "DataInterface", meta = (AllowedClasses = "/Script/ComputeFramework.ComputeDataInterface"))
	TObjectPtr<UClass> Type;
	/** Per-instance settings object. This is created automatically when Type is set. */
	UPROPERTY(VisibleAnywhere, Category = "DataInterface", Instanced, NoClear, meta = (ShowOnlyInnerProperties, EditCondition = "Type!=nullptr", EditConditionHides))
	TObjectPtr<UComputeDataInterface> Settings;
};

/** Top-level description of an editable compute graph: its kernels, binding objects, and data interfaces. */
USTRUCT()
struct FComputeGraphDesc
{
	GENERATED_BODY();

	/** All compute kernels in the graph. */
	UPROPERTY(EditAnywhere, Category = "Kernel")
	TArray<FComputeGraphKernelDesc> Kernels;
	/** All binding objects available for data interfaces to reference. */
	UPROPERTY(EditAnywhere, Category = "BindingObject")
	TArray<FComputeGraphDataBindingObjectDesc> BindingObjects;
	/** All data interfaces that supply read/write functions to kernel pins. */
	UPROPERTY(EditAnywhere, Category = "DataInterface")
	TArray<FComputeGraphDataInterfaceDesc> DataInterfaces;
};

/** Simple data driven compute graph implementation. */
UCLASS(MinimalAPI, meta = (Experimental))
class UEditableComputeGraph : public UComputeGraph
{
	GENERATED_BODY()

protected:
#if WITH_EDITORONLY_DATA
	/** The graph description. */
	UPROPERTY(EditAnywhere, Category = "Graph", meta = (EditInline))
	FComputeGraphDesc GraphDescription;
#endif

#if WITH_EDITOR
public:
	/** Returns the graph description (read-only and mutable overloads). */
	FComputeGraphDesc const& GetGraphDescription() const { return GraphDescription; }
	FComputeGraphDesc& GetGraphDescription() { return GraphDescription; }

	/** Rebuild the compute graph topology and trigger shader compilation. Called by the asset editor Compile button. */
	UE_API void RebuildGraph();

	/** Fired on the game thread for each kernel that finishes compiling. */
	FOnComputeGraphCompileOutput OnCompileOutputChanged;

protected:
	//~ Begin UObject Interface.
	void PostInitProperties() override;
	void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
	void PostTransacted(const FTransactionObjectEvent& TransactionEvent) override;
	//~ End UObject Interface.

	//~ Begin UComputeGraph Interface.
	void OnKernelCompilationComplete(int32 InKernelIndex, FComputeKernelCompileResults const& InCompileResults) override;
	//~ End UComputeGraph Interface.

private:
	/** Translates GraphDescription into a UComputeGraph topology and submits kernels for compilation. */
	void Rebuild();

	/** Maps compiled kernel index as seen by OnKernelCompilationComplete() to a kernel display name. Populated by Rebuild(). */
	TArray<FName> KernelIndexToName;

#endif // WITH_EDITOR
};

#undef UE_API
