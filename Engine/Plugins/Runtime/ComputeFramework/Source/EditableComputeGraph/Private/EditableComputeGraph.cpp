// Copyright Epic Games, Inc. All Rights Reserved.

#include "ComputeFramework/EditableComputeGraph.h"

#include "Components/SceneComponent.h"
#include "ComputeFramework/ComputeDataInterface.h"
#include "ComputeFramework/ComputeDataInterfaceDispatch.h"
#include "ComputeFramework/ComputeGraphBuilder.h"
#include "ComputeFramework/ComputeKernelCompileResult.h"
#include "ComputeFramework/ComputeKernelSource.h"
#include "Misc/TransactionObjectEvent.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(EditableComputeGraph)

#if WITH_EDITOR

void UEditableComputeGraph::PostInitProperties()
{
	Super::PostInitProperties();

	if (!HasAnyFlags(RF_ClassDefaultObject))
	{
		// Every graph must have a binding object.
		if (GraphDescription.BindingObjects.Num() == 0)
		{
			FComputeGraphDataBindingObjectDesc& BindingObject = GraphDescription.BindingObjects.AddDefaulted_GetRef();
			BindingObject.Name = FName(TEXT("Binding"));
			BindingObject.Type = USceneComponent::StaticClass();
		}
		// Every graph must have a Dispatch data interface (provides execution + thread count).
		const bool bHasDispatch = GraphDescription.DataInterfaces.ContainsByPredicate([](FComputeGraphDataInterfaceDesc const& D) { return D.Type == UComputeDataInterfaceDispatch::StaticClass(); });
		if (!bHasDispatch)
		{
			FComputeGraphDataInterfaceDesc& Dispatch = GraphDescription.DataInterfaces.AddDefaulted_GetRef();
			Dispatch.Name = FName(TEXT("Dispatch"));
			Dispatch.BindingObjectName = GraphDescription.BindingObjects[0].Name;
			Dispatch.Type = UComputeDataInterfaceDispatch::StaticClass();
		}
	}
}

void UEditableComputeGraph::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	if (PropertyChangedEvent.MemberProperty && PropertyChangedEvent.Property)
	{
		if (PropertyChangedEvent.MemberProperty->GetFName() == GET_MEMBER_NAME_CHECKED(UEditableComputeGraph, GraphDescription))
		{
			// Skip auto-rebuild for pure name changes. Renaming a kernel, interface or binding object is a label edit that doesn't alter HLSL or graph structure.
			// The editor applies cross-reference fixups in OnFinishedChangingProperties (which fires after this), so rebuilding here would see stale pin names.
			// The explicit compile button handles recompilation after fixups are applied.
			static const FName Name("Name");
			if (PropertyChangedEvent.Property->GetFName() != Name)
			{
				OnCompileOutputChanged.Broadcast({});
				Rebuild();
				FString ValidationErrors;
				if (!ValidateGraph(&ValidationErrors))
				{
					FComputeKernelCompileMessage Msg;
					Msg.Type = FComputeKernelCompileMessage::EMessageType::Error;
					Msg.Text = ValidationErrors.TrimEnd();
					OnCompileOutputChanged.Broadcast({ Msg });
				}
				else
				{
					UpdateResources();
				}
			}
		}
	}
}

void UEditableComputeGraph::PostTransacted(const FTransactionObjectEvent& TransactionEvent)
{
	Super::PostTransacted(TransactionEvent);
	if (TransactionEvent.GetEventType() == ETransactionObjectEventType::UndoRedo)
	{
		RebuildGraph();
	}
}

void UEditableComputeGraph::OnKernelCompilationComplete(int32 InKernelIndex, FComputeKernelCompileResults const& InCompileResults)
{
	const FName KernelName = KernelIndexToName.IsValidIndex(InKernelIndex) ? KernelIndexToName[InKernelIndex] : NAME_None;
	if (KernelName.IsNone() || InCompileResults.Messages.IsEmpty())
	{
		OnCompileOutputChanged.Broadcast(InCompileResults.Messages);
		return;
	}

	// Add the kernel name to each message.
	const FString Prefix = FString::Printf(TEXT("[%s] "), *KernelName.ToString());
	TArray<FComputeKernelCompileMessage> Tagged = InCompileResults.Messages;
	for (FComputeKernelCompileMessage& Msg : Tagged)
	{
		Msg.Text = Prefix + Msg.Text;
	}
	
	OnCompileOutputChanged.Broadcast(Tagged);
}

void UEditableComputeGraph::RebuildGraph()
{
	OnCompileOutputChanged.Broadcast({});
	Rebuild();
	FString ValidationErrors;
	if (!ValidateGraph(&ValidationErrors))
	{
		FComputeKernelCompileMessage Msg;
		Msg.Type = FComputeKernelCompileMessage::EMessageType::Error;
		Msg.Text = ValidationErrors.TrimEnd();
		OnCompileOutputChanged.Broadcast({ Msg });
		return;
	}
	UpdateResources();
}

void UEditableComputeGraph::Rebuild()
{
	FComputeGraphBuilder Builder;

	// Create text based kernel sources and add to the builder.
	KernelIndexToName.Reset();
	TMap<FName, FKernelHandle> KernelHandleMap;
	for (FComputeGraphKernelDesc const& KernelDesc : GraphDescription.Kernels)
	{
		if (KernelDesc.Name.IsNone() || KernelDesc.SourceText.IsEmpty() || KernelDesc.EntryPoint.IsEmpty())
		{
			continue;
		}

		UComputeKernelSourceWithText* Source = NewObject<UComputeKernelSourceWithText>(this);
		Source->SourceText = KernelDesc.SourceText;
		Source->EntryPoint = KernelDesc.EntryPoint;
		Source->GroupSize  = KernelDesc.GroupSize;

		KernelHandleMap.Add(KernelDesc.Name, Builder.AddKernel(Source, KernelDesc.Name));
		KernelIndexToName.Add(KernelDesc.Name);
	}

	// Binding object names.
	TMap<FName, UClass*> BindingClassMap;
	for (FComputeGraphDataBindingObjectDesc const& BindingObjectDesc : GraphDescription.BindingObjects)
	{
		if (!BindingObjectDesc.Name.IsNone() && BindingObjectDesc.Type)
		{
			BindingClassMap.Add(BindingObjectDesc.Name, BindingObjectDesc.Type);
		}
	}

	// Data interfaces.
	TMap<FName, FInterfaceHandle> InterfaceHandleMap;
	for (FComputeGraphDataInterfaceDesc& InterfaceDesc : GraphDescription.DataInterfaces)
	{
		if (InterfaceDesc.BindingObjectName.IsNone() || InterfaceDesc.Type == nullptr)
		{
			continue;
		}

		UClass* BindingClass = BindingClassMap.FindRef(InterfaceDesc.BindingObjectName);
		if (!BindingClass)
		{
			continue;
		}

		// Preserve any existing settings.
		if (InterfaceDesc.Type->ChildProperties)
		{
			if (!InterfaceDesc.Settings || !InterfaceDesc.Settings->IsA(InterfaceDesc.Type))
			{
				InterfaceDesc.Settings = NewObject<UComputeDataInterface>(this, InterfaceDesc.Type);
			}
		}
		else
		{
			InterfaceDesc.Settings = nullptr;
		}

		UComputeDataInterface* DataInterface = InterfaceDesc.Settings
			? DuplicateObject<UComputeDataInterface>(InterfaceDesc.Settings, this)
			: NewObject<UComputeDataInterface>(this, InterfaceDesc.Type);

		InterfaceHandleMap.Add(InterfaceDesc.Name, Builder.AddDataInterface(DataInterface, BindingClass, InterfaceDesc.Name));
	}

	// Connect pins. Invalid handles are silently skipped by the builder.
	for (FComputeGraphKernelDesc const& KernelDesc : GraphDescription.Kernels)
	{
		const FKernelHandle KernelHandle = KernelHandleMap.FindRef(KernelDesc.Name);

		for (FKernelPin const& Pin : KernelDesc.Inputs)
		{
			Builder.ConnectInput(KernelHandle, Pin.KernelFunctionName, InterfaceHandleMap.FindRef(Pin.DataInterfaceName), Pin.DataInterfaceFunctionName);
		}

		for (FKernelPin const& Pin : KernelDesc.Outputs)
		{
			Builder.ConnectOutput(KernelHandle, Pin.KernelFunctionName, InterfaceHandleMap.FindRef(Pin.DataInterfaceName), Pin.DataInterfaceFunctionName);
		}
	}

	Builder.Build(*this, *this);
}

#endif
