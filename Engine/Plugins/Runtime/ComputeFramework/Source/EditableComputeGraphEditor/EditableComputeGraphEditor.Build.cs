// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class EditableComputeGraphEditor : ModuleRules
	{
		public EditableComputeGraphEditor(ReadOnlyTargetRules Target) : base(Target)
		{
			PrivateDependencyModuleNames.AddRange(
				new string[]
				{
					"AssetDefinition",
					"ComputeDataInterface",
					"ComputeFramework",
					"EditableComputeGraph",
					"Core",
					"CoreUObject",
					"Engine",
					"InputCore",
					"PropertyEditor",
					"Slate",
					"SlateCore",
					"ToolMenus",
					"UnrealEd",
				}
			);
		}
	}
}
