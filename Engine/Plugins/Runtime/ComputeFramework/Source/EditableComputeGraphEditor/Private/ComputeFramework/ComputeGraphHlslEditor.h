// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/SCompoundWidget.h"

class FComputeGraphHlslSyntaxHighlighter;
class SMultiLineEditableTextBox;
class STextBlock;
class UEditableComputeGraph;

DECLARE_DELEGATE_TwoParams(FOnComputeGraphHlslTextCommitted, FName /* KernelName */, FString const& /* NewText */);
DECLARE_DELEGATE(FOnComputeGraphHlslTextChanged);
DECLARE_DELEGATE(FOnComputeGraphHlslCompileRequested);

/**
 * HLSL editor for the EditableComputeGraph asset editor.
 * Displays the source text of the currently selected kernel in a monospace multi-line text box. 
 * User commits the text by changing focus or F8 to compile.
 */
class SComputeGraphHlslEditor : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SComputeGraphHlslEditor) {}
		SLATE_ARGUMENT(UEditableComputeGraph*, Asset)
		SLATE_EVENT(FOnComputeGraphHlslTextCommitted, OnTextCommitted_WithKernel)
		SLATE_EVENT(FOnComputeGraphHlslTextChanged, OnTextChanged)
		SLATE_EVENT(FOnComputeGraphHlslCompileRequested, OnCompileRequested)
	SLATE_END_ARGS()

	void Construct(FArguments const& InArgs);

	/** Switch the editor to display a different kernel's source text. */
	void SetActiveKernel(FName KernelName);

	/** Highlight the given DI_ function names with the bound-function colour. */
	void SetBoundFunctions(TArray<FString> const& Names);

	/** Flush any in-progress edit. */
	void CommitCurrentText();

private:
	/** Override key down handler to intercept F8 for triggering a compile request. */
	FReply OnKeyDown(FGeometry const& MyGeometry, FKeyEvent const& InKeyEvent) override;
	/** First chance key handler wired to the text box to intercept Tab before Slate focus traversal. */
	FReply OnTextBoxKeyDown(FGeometry const& MyGeometry, FKeyEvent const& InKeyEvent);

	/** Fires the commit delegate when the user finishes editing. */
	void OnTextCommitted(FText const& NewText, ETextCommit::Type CommitType);
	/** Notifies the toolkit that the text has changed so it can mark the asset dirty. */
	void OnTextChangedInternal(FText const& NewText);

	/** The asset being edited. */
	TWeakObjectPtr<UEditableComputeGraph> Asset;

	/** Fires with the kernel name and new source text when the user commits an edit. */
	FOnComputeGraphHlslTextCommitted OnTextCommitted_WithKernel;
	/** Fires whenever the text box content changes. */
	FOnComputeGraphHlslTextChanged OnTextChanged;
	/** Fires when the user requests a compile (F8). */
	FOnComputeGraphHlslCompileRequested OnCompileRequested;

	/** Name of the kernel whose source text is currently displayed. */
	FName ActiveKernelName;
	/** The multi-line text box used for HLSL input. */
	TSharedPtr<SMultiLineEditableTextBox> TextBox;
	/** Shown in place of the text box when no kernel is selected. */
	TSharedPtr<STextBlock> PlaceholderText;

	/** Syntax highlighter that colours keywords, DI_ calls, and bound functions. */
	TSharedPtr<FComputeGraphHlslSyntaxHighlighter> SyntaxHighlighter;
};
