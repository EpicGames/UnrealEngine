// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Text/HLSLSyntaxHighlighterMarshaller.h"

/**
 * HLSL syntax highlighter for the EditableComputeGraph kernel editor.
 * Extends the base HLSL highlighter with an extra colour for DI_ function names that are currently bound in the kernel's pin list.
 */
class FComputeGraphHlslSyntaxHighlighter : public FHLSLSyntaxHighlighterMarshaller
{
public:
	/** Creates the highlighter. */
	static TSharedRef<FComputeGraphHlslSyntaxHighlighter> Create(FSyntaxTextStyle const& InSyntaxTextStyle, FTextBlockStyle const& InBoundFunctionStyle);

	/** Update the set of bound DI_ function names and trigger a re-highlight. */
	void SetBoundFunctions(TArray<FString> const& Names);

protected:
	/** Applies bound-function colouring on top of the base HLSL token styles. */
	FTextLayout::FNewLineData ProcessTokenizedLine(
		ISyntaxTokenizer::FTokenizedLine const& TokenizedLine,
		int32 const& LineNumber,
		FString const& SourceString,
		EParseState& ParseState) override;

private:
	/** Private constructor. Use Create(). */
	FComputeGraphHlslSyntaxHighlighter(
		TSharedPtr<ISyntaxTokenizer> InTokenizer,
		FSyntaxTextStyle const& InSyntaxTextStyle,
		FTextBlockStyle const& InBoundFunctionStyle);

	/** Text style applied to tokens that match a name in BoundFunctionNames. */
	FTextBlockStyle BoundFunctionStyle;
	/** DI_ function names currently wired to a kernel pin. */
	TSet<FString> BoundFunctionNames;
};
