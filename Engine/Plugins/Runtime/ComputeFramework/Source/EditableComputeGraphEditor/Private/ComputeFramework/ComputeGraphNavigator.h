// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "ComputeFramework/EditableComputeGraphDetailCustomization.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/Views/STreeView.h"

class UEditableComputeGraph;
struct FNavigatorItem;
using FNavigatorItemPtr = TSharedPtr<FNavigatorItem>;

/** The category of item shown in the compute graph toolkit navigator panel. */
enum class EComputeGraphItemKind : uint8
{
	None,
	Kernel,
	Interface,
	BindingObject,
};

/** Shared selection state between the navigator and the toolkit detail customization. */
struct FComputeGraphEditorSelection
{
	/** Item type. */
	EComputeGraphItemKind Kind = EComputeGraphItemKind::None;
	/** Index of the item within its kind array. */
	int32 Index = INDEX_NONE;
};

/** Fired when the user clicks a leaf item. Index is the position in the relevant array. */
DECLARE_DELEGATE_ThreeParams(FOnComputeGraphItemSelected, EComputeGraphItemKind, int32 /* Index */, FName /* Name */);

/** Fired when the user clicks a section [+] button. */
DECLARE_DELEGATE_OneParam(FOnComputeGraphAddItem, EComputeGraphItemKind);

/** Fired when the user selects Delete from the context menu. */
DECLARE_DELEGATE_TwoParams(FOnComputeGraphDeleteItem, EComputeGraphItemKind, int32 /* Index */);

/** Fired when the user selects Duplicate from the context menu. */
DECLARE_DELEGATE_TwoParams(FOnComputeGraphDuplicateItem, EComputeGraphItemKind, int32 /* Index */);

/** Fired when the user commits a rename (inline text or context menu → Rename). */
DECLARE_DELEGATE_ThreeParams(FOnComputeGraphRenameItem, EComputeGraphItemKind, int32 /* Index */, FName /* NewName */);

/** Kept for HLSL editor wiring: fires when a kernel is selected. */
DECLARE_DELEGATE_OneParam(FOnComputeGraphKernelSelected, FName /* KernelName */);

/**
 * Item navigator for the EditableComputeGraph asset editor.
 * Supports management of Binding Objects, Data Interfaces and Kernels.
 * Styled to match the Blueprint item panel.
 */
class SComputeGraphNavigator : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SComputeGraphNavigator) {}
		SLATE_ARGUMENT(UEditableComputeGraph*, Asset)
		SLATE_EVENT(FOnComputeGraphItemSelected, OnItemSelected)
		SLATE_EVENT(FOnComputeGraphAddItem, OnAddItem)
		SLATE_EVENT(FOnComputeGraphDeleteItem, OnDeleteItem)
		SLATE_EVENT(FOnComputeGraphDuplicateItem, OnDuplicateItem)
		SLATE_EVENT(FOnComputeGraphRenameItem, OnRenameItem)
		SLATE_EVENT(FOnComputeGraphKernelSelected, OnKernelSelected)
	SLATE_END_ARGS()

	void Construct(FArguments const& InArgs);

	/** Rebuild the tree from the current asset description and reapply selection/expansion. */
	void Refresh();

	/** Programmatically select an item (e.g. after Add or Duplicate). Pass INDEX_NONE to clear. */
	void SetSelectedItem(EComputeGraphItemKind Kind, int32 Index);

private:
	/** Rebuilds RootItems from the current asset description. */
	void RebuildItems();
	/** Builds either a section header row or a leaf item row for the tree view. */
	TSharedRef<ITableRow> GenerateRow(FNavigatorItemPtr Item, TSharedRef<STableViewBase> const& OwnerTable);
	/** Fills OutChildren from the item's Children array (header items only). */
	void OnGetChildren(FNavigatorItemPtr Item, TArray<FNavigatorItemPtr>& OutChildren);
	/** Builds the right-click context menu for a selected leaf item. */
	TSharedPtr<SWidget> BuildContextMenu();

	/** Override key down handler to intercept F2 for inline rename on the selected item. */
	FReply OnKeyDown(FGeometry const& MyGeometry, FKeyEvent const& InKeyEvent) override;

	// Delegate functions.
	void OnSelectionChanged(FNavigatorItemPtr Item, ESelectInfo::Type SelectInfo);
	void OnExpansionChanged(FNavigatorItemPtr Item, bool bExpanded);
	void OnAddItemClicked(EComputeGraphItemKind Kind);
	void OnRenameItemCommitted(EComputeGraphItemKind Kind, int32 Index, FName NewName);
	void OnDeleteSelected();
	void OnDuplicateSelected();
	void OnRenameSelected();

	/** Returns false and sets OutError if the candidate name is empty or already in use. */
	bool VerifyRenameText(FText const& NewText, FText& OutError, EComputeGraphItemKind Kind, int32 Index) const;

	/** The asset being navigated; held weakly since this widget is not a GC object. */
	TWeakObjectPtr<UEditableComputeGraph> Asset;

	/** The tree view widget. */
	TSharedPtr<STreeView<FNavigatorItemPtr>> TreeView;
	/** Root items — one per section header; each carries its leaf children. */
	TArray<FNavigatorItemPtr> RootItems;
	/** Sections that have been manually collapsed; all others are expanded. */
	TSet<EComputeGraphItemKind> CollapsedSections;

	/** Current selection; mirrored here so the context menu can read it without a tree query. */
	FComputeGraphEditorSelection Selection;

	// Delegate objects.
	FOnComputeGraphItemSelected OnItemSelected;
	FOnComputeGraphAddItem OnAddItem;
	FOnComputeGraphDeleteItem OnDeleteItem;
	FOnComputeGraphDuplicateItem OnDuplicateItem;
	FOnComputeGraphRenameItem OnRenameItem;
	FOnComputeGraphKernelSelected OnKernelSelected;
};
