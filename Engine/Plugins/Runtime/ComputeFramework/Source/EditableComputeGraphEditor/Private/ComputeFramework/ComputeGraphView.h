// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/SLeafWidget.h"

class UEditableComputeGraph;

DECLARE_DELEGATE_TwoParams(FOnComputeGraphNodeClicked, FName /*NodeName*/, bool /*bIsKernel*/);

/**
 * Read only graph view for the EditableComputeGraph asset editor.
 * Each connected pin pair is drawn as a spline.
 * The selected kernels and data interfaces can be highlighted along with a dimmer highlight on their neighboring nodes.
 * Clicking on nodes in the graph view will select them in the toolkit.
 */
class SComputeGraphView : public SLeafWidget
{
public:
	SLATE_BEGIN_ARGS(SComputeGraphView) {}
		SLATE_ARGUMENT(UEditableComputeGraph*, Asset)
		SLATE_EVENT(FOnComputeGraphNodeClicked, OnNodeClicked)
	SLATE_END_ARGS()

	void Construct(FArguments const& InArgs);

	/** Highlight edges connected to a specific kernel. */
	void SetHighlightedKernel(FName KernelName);

	/** Highlight a specific data interface node. */
	void SetHighlightedInterface(FName InterfaceName);

	/** Rebuild the visual layout from the current asset description. */
	void Refresh();

protected:
	//~ Begin SWidget interface.
	int32 OnPaint(FPaintArgs const& Args, FGeometry const& AllottedGeometry, FSlateRect const& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, FWidgetStyle const& InWidgetStyle, bool bParentEnabled) const override;
	FVector2D ComputeDesiredSize(float LayoutScaleMultiplier) const override;
	FReply OnMouseButtonDown(FGeometry const& MyGeometry, FPointerEvent const& MouseEvent) override;
	//~ End SWidget interface.

private:
	/** Rebuilds NodeLayouts, Edges, centre lookup maps, and cached size metrics from the current asset description. */
	void RebuildLayout();

	/** Position and rank of one node in the graph view. */
	struct FNodeLayout
	{
		FName Name;
		FVector2D Centre; // Local widget space (pixels, pre-DPI-scale).
		int32 Rank = 0; // Column index: 0 = leftmost interfaces, increasing toward outputs.
		bool bKernel = false;
	};

	/** One directed connection between a kernel pin and a data interface. */
	struct FEdge
	{
		FName KernelName;
		FName InterfaceName;
		FString KernelFn;
		FString InterfaceFn;
		bool bOrphaned = false; // True when the pin has no valid binding; drawn red.
		bool bIsOutput = false; // True for kernel output pins (kernel → interface direction).
	};

	/** The asset being visualised; held weakly since this widget is not a GC object. */
	TWeakObjectPtr<UEditableComputeGraph> Asset;

	/** Flat list of node positions, rebuilt by RebuildLayout(). */
	TArray<FNodeLayout> NodeLayouts;
	/** Flat list of edges (one per kernel pin), rebuilt by RebuildLayout(). */
	TArray<FEdge> Edges;
	/** Centre position lookup for interface nodes, keyed by name. Rebuilt by RebuildLayout(). */
	TMap<FName, FVector2D> InterfaceCentreMap;
	/** Centre position lookup for kernel nodes, keyed by name. Rebuilt by RebuildLayout(). */
	TMap<FName, FVector2D> KernelCentreMap;

	/** Name of the kernel node to draw highlighted. NAME_None when nothing is selected. */
	FName HighlightedKernel;
	/** Name of the interface node to draw highlighted. NAME_None when nothing is selected. */
	FName HighlightedInterface;
	/** Highest rank value present in NodeLayouts that drives the widget width. */
	int32 MaxRank = 0;
	/** Largest number of nodes sharing a rank column that drives the widget height. */
	int32 MaxNodesInAnyRank = 0;

	/** Fired when the user clicks a node. */
	FOnComputeGraphNodeClicked OnNodeClicked;
};
