// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AssetDefinitionDefault.h"

#include "EditableComputeGraphAssetDefinition.generated.h"

UCLASS()
class UAssetDefinition_EditableComputeGraph : public UAssetDefinitionDefault
{
	GENERATED_BODY()

	//~ Begin UAssetDefinition Interface.
	virtual FText GetAssetDisplayName() const override;
	virtual FLinearColor GetAssetColor() const override;
	virtual TSoftClassPtr<UObject> GetAssetClass() const override;
	virtual TConstArrayView<FAssetCategoryPath> GetAssetCategories() const override;
	virtual EAssetCommandResult OpenAssets(const FAssetOpenArgs& OpenArgs) const override;
	//~ End UAssetDefinition Interface.
};
