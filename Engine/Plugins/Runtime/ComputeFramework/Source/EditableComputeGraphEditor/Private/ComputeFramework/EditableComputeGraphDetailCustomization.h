// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IDetailCustomization.h"

struct FComputeGraphEditorSelection;

/**
 * Detail Customization for UEditableComputeGraph.
 * Hides the full "Graph" category and instead exposes only the current array element that the user has selected in the navigator, using property handle navigation.
 */
class FEditableComputeGraphDetailCustomization : public IDetailCustomization
{
public:
	static TSharedRef<IDetailCustomization> MakeInstance(TSharedPtr<FComputeGraphEditorSelection> InSelection);

protected:
	//~ Begin IDetailCustomization Interface.
	void CustomizeDetails(IDetailLayoutBuilder& DetailBuilder) override;
	//~ End IDetailCustomization Interface.

	/** Current selection. */
	TSharedPtr<FComputeGraphEditorSelection> Selection;
};
