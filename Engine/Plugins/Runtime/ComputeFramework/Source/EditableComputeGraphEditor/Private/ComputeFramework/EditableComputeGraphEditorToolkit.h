// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Textures/SlateIcon.h"
#include "Toolkits/AssetEditorToolkit.h"
#include "UObject/GCObject.h"
#include "Widgets/Views/SListView.h"

enum class EComputeGraphItemKind : uint8;
struct FComputeGraphEditorSelection;
struct FComputeKernelCompileMessage;
class IDetailsView;
class SComputeGraphHlslEditor;
class SComputeGraphNavigator;
class SComputeGraphView;
class UEditableComputeGraph;

/** Custom asset editor for UEditableComputeGraph. */
class FEditableComputeGraphEditorToolkit : public FAssetEditorToolkit, public FGCObject
{
public:
	static TSharedRef<FEditableComputeGraphEditorToolkit> Create(UEditableComputeGraph* InAsset, EToolkitMode::Type InMode,	TSharedPtr<IToolkitHost> InToolkitHost);

	/** Returns the asset being edited. */
	UEditableComputeGraph* GetAsset() const { return Asset; }

	/** Flushes any uncommitted HLSL text then triggers a full graph rebuild and shader compile. */
	void CompileGraph();
	/** Whether the graph is compiled and up to date, needs recompile, or has errors. */
	enum class ECompileState { UpToDate, NeedsCompile, Broken };
	/** Inspects the graph description to determine the current compile state without recompiling. */
	ECompileState GetCompileState() const;
	/** Returns the toolbar icon matching the current compile state. */
	FSlateIcon GetCompileStatusIcon() const;

protected:
	//~ Begin FAssetEditorToolkit Interface.
	FName GetToolkitFName() const override;
	FText GetBaseToolkitName() const override;
	FString GetWorldCentricTabPrefix() const override;
	FLinearColor GetWorldCentricTabColorScale() const override;
	void RegisterTabSpawners(TSharedRef<FTabManager> const& InTabManager) override;
	void UnregisterTabSpawners(TSharedRef<FTabManager> const& InTabManager) override;
	//~ End FAssetEditorToolkit Interface.

	//~ Begin FGCObject Interface.
	void AddReferencedObjects(FReferenceCollector& Collector) override;
	FString GetReferencerName() const override;
	//~ End FGCObject Interface.

	/** Updates the details panel and HLSL editor to reflect the newly selected item. */
	void OnItemSelected(EComputeGraphItemKind Kind, int32 Index, FName Name);
	/** Appends a new kernel, interface, or binding object with a generated unique name. */
	void OnAddItem(EComputeGraphItemKind Kind);
	/** Removes the item at Index and clears any dangling pin references to it. */
	void OnDeleteItem(EComputeGraphItemKind Kind, int32 Index);
	/** Appends a deep copy of the item at Index with a "_Copy" suffix on its name. */
	void OnDuplicateItem(EComputeGraphItemKind Kind, int32 Index);
	/** Renames the item at Index and propagates the new name to any referencing pins. */
	void OnRenameItem(EComputeGraphItemKind Kind, int32 Index, FName NewName);
	/** Loads the selected kernel's source into the HLSL editor. */
	void OnKernelSelected(FName KernelName);
	/** Syncs pin lists from the new HLSL text and marks the graph dirty. */
	void OnHlslTextCommitted(FName KernelName, FString const& NewText);
	/** Selects the corresponding kernel or interface in the navigator when a graph node is clicked. */
	void OnGraphNodeClicked(FName NodeName, bool bIsKernel);
	/** Appends newly arrived compile messages to the output list view. */
	void OnCompileOutputChanged(TArray<FComputeKernelCompileMessage> const& NewMessages);
	/** Navigates the navigator to the kernel associated with the clicked output message. */
	void OnOutputSelectionChanged(TSharedPtr<FComputeKernelCompileMessage> Selected, ESelectInfo::Type SelectInfo);

private:
	/** Initialises the toolkit: creates the layout, registers tab spawners, and binds all callbacks. */
	void Init(UEditableComputeGraph* InAsset, EToolkitMode::Type InMode, TSharedPtr<IToolkitHost> InToolkitHost);

	// Tab spawn functions.
	TSharedRef<SDockTab> SpawnTab_Navigator(FSpawnTabArgs const& Args);
	TSharedRef<SDockTab> SpawnTab_HlslEditor(FSpawnTabArgs const& Args);
	TSharedRef<SDockTab> SpawnTab_GraphView(FSpawnTabArgs const& Args);
	TSharedRef<SDockTab> SpawnTab_Details(FSpawnTabArgs const& Args);
	TSharedRef<SDockTab> SpawnTab_Output(FSpawnTabArgs const& Args);

	/** Builds one row in the compile output list view. */
	TSharedRef<ITableRow> GenerateOutputRow(TSharedPtr<FComputeKernelCompileMessage> Message, TSharedRef<STableViewBase> const& OwnerTable);

	/** Adds the compile button and status icon to the asset editor toolbar. */
	void ExtendToolbar();
	/** Pushes the current kernel's bound pin names into the HLSL editor for syntax highlighting. */
	void RefreshHlslBoundFunctions(FName KernelName);
	/** Marks the graph dirty, refreshes the navigator, graph view, and details panel. */
	void RefreshNavigatorAndDetails();

	/** Unique name helper. Returns "Base", "Base1", "Base2" etc while avoiding names in Existing. */
	static FName GenerateUniqueName(TArray<FName> const& Existing, FStringView Base);

private:
	/** The asset being edited. */
	TObjectPtr<UEditableComputeGraph> Asset;

	/** Tree-view panel listing kernels, interfaces, and binding objects. */
	TSharedPtr<SComputeGraphNavigator> NavigatorWidget;
	/** Multi-line HLSL editor panel. */
	TSharedPtr<SComputeGraphHlslEditor> HlslEditorWidget;
	/** Bipartite graph visualisation panel. */
	TSharedPtr<SComputeGraphView> GraphViewWidget;
	/** Details panel showing per-item properties. */
	TSharedPtr<IDetailsView> DetailsView;

	/** The list view widget in the Output tab. */
	TSharedPtr<SListView<TSharedPtr<FComputeKernelCompileMessage>>> OutputListView;
	/** Messages currently displayed in the Output tab; repopulated on each compile. */
	TArray<TSharedPtr<FComputeKernelCompileMessage>> OutputMessages;
	/** Bound to UEditableComputeGraph::OnCompileOutputChanged to refresh the output tab. */
	FDelegateHandle CompileOutputHandle;

	/** Shared with FEditableComputeGraphDetailCustomization; updated on every selection change. */
	TSharedPtr<FComputeGraphEditorSelection> EditorSelection;

	/** True whenever graph data has been mutated since the last successful compile. */
	bool bDirty = true;

	/** Name of the currently selected Interface or BindingObject, captured at selection time so OnFinishedChangingProperties can detect in-panel renames and apply cross-reference fixups. */
	FName CachedSelectedItemName = NAME_None;

	// Tab Id FNames.
	static FName const TabId_Navigator;
	static FName const TabId_HlslEditor;
	static FName const TabId_GraphView;
	static FName const TabId_Details;
	static FName const TabId_Output;
};
