// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

/** Helper functions for finding function bindings in HLSL source. */
class FHlslExternalPinParser
{
public:
	/** Result of one DI_ call site found by FindExternalPins(). */
	struct FPinDeclaration
	{
		/** Full function name including the DI_ prefix, e.g. "DI_ReadValue". */
		FString FunctionName;
		/** True when the call is considered to be an output. */
		bool bIsOutput = false;
	};

	/**
	 * Scan SourceText for DataInterface function calls.
	 * The convention used by UEditableComputeGraph kernel authoring is that DataInterface function calls have the "DI_" prefix in the kernel.
	 * These will be presented in the toolkit UI for binding.
	 * The input/output direction is inferred from call-site context. If DI_*() is a standalone statement (only whitespace precedes it on the same line) then
	 * the call discards its return value and we make the assumption it is an output pin.
	 */
	static TArray<FPinDeclaration> FindExternalPins(FStringView SourceText);

	/**
	 * Scan SourceText for top level function definitions. These are signatures followed by a body.
	 * We returns the bare function names in declaration order.
	 * This is usful for finding potential kernel EntryPoints.
	 */
	static TArray<FString> FindFunctionDefinitions(FStringView SourceText);

	/**
	 * Returns true if FunctionName appears in SourceText as a whole-word identifier followed by '('.
	 */
	static bool FunctionExistsInText(FStringView SourceText, FString const& FunctionName);
};
