// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IPropertyTypeCustomization.h"
#include "Styling/SlateColor.h"

class IPropertyHandle;

/**
 * Details panel customization for FKernelPin.
 * Holds combo boxes for populating the DataInterface and function.
 */
class FKernelPinCustomization : public IPropertyTypeCustomization
{
public:
	static TSharedRef<IPropertyTypeCustomization> MakeInstance();

protected:
	//~ Begin IPropertyTypeCustomization Interface.
	void CustomizeHeader(TSharedRef<IPropertyHandle> StructPropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& CustomizationUtils) override;
	void CustomizeChildren(TSharedRef<IPropertyHandle> StructPropertyHandle, IDetailChildrenBuilder& ChildBuilder, IPropertyTypeCustomizationUtils& CustomizationUtils) override;
	//~ End IPropertyTypeCustomization Interface.

private:
	/** Rebuilds the kernel function combo options from the owning kernel's HLSL source. */
	void RefreshKernelFunctionNameOptions();
	/** Rebuilds the data interface combo options from the graph's interface list. */
	void RefreshInterfaceNameOptions();
	/** Rebuilds the DI function combo options from the currently selected interface's declared functions. */
	void RefreshFunctionNameOptions();

	/** Returns true when the pin is orphaned or any required field is unset. */
	bool NeedsAttention() const;
	/** Text shown in the collapsed header row: "KernelFn → Interface::DIFn". */
	FText GetHeaderSummaryText() const;
	/** Orange when NeedsAttention(), foreground colour otherwise. */
	FSlateColor GetHeaderSummaryColor() const;
	/** Tooltip explaining why the pin needs attention (orphaned or incomplete). */
	FText GetOrphanedTooltipText() const;
	/** Current value of KernelFunctionName as display text. */
	FText GetCurrentKernelFunctionNameText() const;
	/** Current value of DataInterfaceName as display text. */
	FText GetCurrentInterfaceNameText() const;
	/** Current value of DataInterfaceFunctionName as display text. */
	FText GetCurrentFunctionNameText() const;

	/** Writes the selected kernel function name back to the property. */
	void OnKernelFunctionSelected(TSharedPtr<FString> NewValue, ESelectInfo::Type SelectInfo);
	/** Writes the selected interface name back to the property and refreshes the function list. */
	void OnInterfaceSelected(TSharedPtr<FName> NewValue, ESelectInfo::Type SelectInfo);
	/** Writes the selected DI function name back to the property. */
	void OnFunctionSelected(TSharedPtr<FString> NewValue, ESelectInfo::Type SelectInfo);

	/** True when customizing an input pin. False for an output pin. */
	bool bIsInput = true;

	/** Handle for the FKernelPin struct itself. */
	TSharedPtr<IPropertyHandle> StructHandle;
	/** Handle for FKernelPin::KernelFunctionName. */
	TSharedPtr<IPropertyHandle> KernelFnHandle;
	/** Handle for FKernelPin::DataInterfaceName. */
	TSharedPtr<IPropertyHandle> InterfaceNameHandle;
	/** Handle for FKernelPin::DataInterfaceFunctionName. */
	TSharedPtr<IPropertyHandle> FunctionNameHandle;
	/** Handle for FKernelPin::bOrphaned. */
	TSharedPtr<IPropertyHandle> OrphanedHandle;
	/** Handle for the owning FComputeGraphKernelDesc::SourceText. */
	TSharedPtr<IPropertyHandle> KernelSourceHandle;
	/** Handle for the owning FComputeGraphKernelDesc::EntryPoint. */
	TSharedPtr<IPropertyHandle> KernelEntryPointHandle;

	/** Populated by RefreshKernelFunctionNameOptions(). Drives the kernel function combo box. */
	TArray<TSharedPtr<FString>> KernelFunctionNameOptions;
	/** Populated by RefreshInterfaceNameOptions(). Drives the data interface combo box. */
	TArray<TSharedPtr<FName>> InterfaceNameOptions;
	/** Populated by RefreshFunctionNameOptions(). Drives the DI function combo box. */
	TArray<TSharedPtr<FString>> FunctionNameOptions;
};
