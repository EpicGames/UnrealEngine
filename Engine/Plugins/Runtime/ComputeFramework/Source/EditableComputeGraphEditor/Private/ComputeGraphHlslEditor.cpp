// Copyright Epic Games, Inc. All Rights Reserved.

#include "ComputeFramework/ComputeGraphHlslEditor.h"

#include "ComputeFramework/ComputeGraphHlslSyntaxHighlighter.h"
#include "ComputeFramework/EditableComputeGraph.h"
#include "Fonts/SlateFontInfo.h"
#include "Styling/AppStyle.h"
#include "Styling/CoreStyle.h"
#include "Widgets/Input/SMultiLineEditableTextBox.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "ComputeFrameworkEditor"

void SComputeGraphHlslEditor::Construct(FArguments const& InArgs)
{
	Asset = InArgs._Asset;
	OnTextCommitted_WithKernel = InArgs._OnTextCommitted_WithKernel;
	OnTextChanged = InArgs._OnTextChanged;
	OnCompileRequested = InArgs._OnCompileRequested;

	FSlateFontInfo MonoFont = FCoreStyle::GetDefaultFontStyle("Mono", 10);

	// Bound DI_ function names are highlighted with a teal colour.
	FTextBlockStyle BoundFunctionStyle = FAppStyle::Get().GetWidgetStyle<FTextBlockStyle>("SyntaxHighlight.SourceCode.Normal");
	BoundFunctionStyle.SetColorAndOpacity(FLinearColor(0.2f, 0.85f, 0.75f, 1.f));

	SyntaxHighlighter = FComputeGraphHlslSyntaxHighlighter::Create(
		FHLSLSyntaxHighlighterMarshaller::FSyntaxTextStyle(
			FAppStyle::Get().GetWidgetStyle<FTextBlockStyle>("SyntaxHighlight.SourceCode.Normal"),
			FAppStyle::Get().GetWidgetStyle<FTextBlockStyle>("SyntaxHighlight.SourceCode.Operator"),
			FAppStyle::Get().GetWidgetStyle<FTextBlockStyle>("SyntaxHighlight.SourceCode.Keyword"),
			FAppStyle::Get().GetWidgetStyle<FTextBlockStyle>("SyntaxHighlight.SourceCode.String"),
			FAppStyle::Get().GetWidgetStyle<FTextBlockStyle>("SyntaxHighlight.SourceCode.Number"),
			FAppStyle::Get().GetWidgetStyle<FTextBlockStyle>("SyntaxHighlight.SourceCode.Comment"),
			FAppStyle::Get().GetWidgetStyle<FTextBlockStyle>("SyntaxHighlight.SourceCode.PreProcessorKeyword"),
			FAppStyle::Get().GetWidgetStyle<FTextBlockStyle>("SyntaxHighlight.SourceCode.Error")),
		BoundFunctionStyle);

	ChildSlot
	[
		SNew(SVerticalBox)

		// Header bar showing which kernel is being edited.
		+ SVerticalBox::Slot()
		.AutoHeight()
		[
			SNew(SBorder)
			.BorderImage(FAppStyle::Get().GetBrush("ToolPanel.GroupBorder"))
			.Padding(FMargin(8.f, 4.f))
			[
				SAssignNew(PlaceholderText, STextBlock)
				.Text(LOCTEXT("NoKernelSelected", "Select a kernel from the navigator."))
				.Font(FAppStyle::Get().GetFontStyle("PropertyWindow.NormalFont"))
				.ColorAndOpacity(FSlateColor::UseSubduedForeground())
			]
		]

		// Text editor area.
		+ SVerticalBox::Slot()
		.FillHeight(1.f)
		[
			SNew(SBorder)
			.BorderImage(FAppStyle::Get().GetBrush("NoBorder"))
			.Padding(0.f)
			[
				SAssignNew(TextBox, SMultiLineEditableTextBox)
				.Font(MonoFont)
				.Marshaller(SyntaxHighlighter)
				.AutoWrapText(false)
				.AllowMultiLine(true)
				.IsReadOnly(true) // Read-only until a kernel is selected.
				.OnTextCommitted(this, &SComputeGraphHlslEditor::OnTextCommitted)
				.OnTextChanged(this, &SComputeGraphHlslEditor::OnTextChangedInternal)
				.OnKeyDownHandler(this, &SComputeGraphHlslEditor::OnTextBoxKeyDown)
			]
		]
	];
}

void SComputeGraphHlslEditor::SetActiveKernel(FName KernelName)
{
	if (!Asset.IsValid())
	{
		return;
	}

	ActiveKernelName = KernelName;

	// Find the kernel in the description and load its source text.
	FComputeGraphDesc const& Desc = Asset->GetGraphDescription();
	for (FComputeGraphKernelDesc const& KernelDesc : Desc.Kernels)
	{
		if (KernelDesc.Name != KernelName)
		{
			continue;
		}

		if (TextBox.IsValid())
		{
			TextBox->SetText(FText::FromString(KernelDesc.SourceText));
			TextBox->SetIsReadOnly(false);
		}

		if (PlaceholderText.IsValid())
		{
			PlaceholderText->SetText(FText::Format(LOCTEXT("KernelHeader", "Kernel: {0}"), FText::FromName(KernelName)));
		}
		return;
	}

	// Kernel not found. Most likely it was just renamed/deleted.
	if (TextBox.IsValid())
	{
		TextBox->SetText(FText::GetEmpty());
		TextBox->SetIsReadOnly(true);
	}
	if (PlaceholderText.IsValid())
	{
		PlaceholderText->SetText(LOCTEXT("NoKernelSelected", "Select a kernel from the navigator."));
	}
	ActiveKernelName = NAME_None;
}

void SComputeGraphHlslEditor::SetBoundFunctions(TArray<FString> const& Names)
{
	if (SyntaxHighlighter.IsValid())
	{
		SyntaxHighlighter->SetBoundFunctions(Names);
	}
}

void SComputeGraphHlslEditor::OnTextCommitted(FText const& NewText, ETextCommit::Type CommitType)
{
	if (!ActiveKernelName.IsNone() && CommitType != ETextCommit::OnCleared)
	{
		OnTextCommitted_WithKernel.ExecuteIfBound(ActiveKernelName, NewText.ToString());
	}
}

void SComputeGraphHlslEditor::CommitCurrentText()
{
	if (!ActiveKernelName.IsNone() && TextBox.IsValid())
	{
		OnTextCommitted_WithKernel.ExecuteIfBound(ActiveKernelName, TextBox->GetText().ToString());
	}
}

void SComputeGraphHlslEditor::OnTextChangedInternal(FText const& /*NewText*/)
{
	OnTextChanged.ExecuteIfBound();
}

FReply SComputeGraphHlslEditor::OnKeyDown(FGeometry const& MyGeometry, FKeyEvent const& InKeyEvent)
{
	// F8 triggers a compile.
	if (InKeyEvent.GetKey() == EKeys::F8)
	{
		OnCompileRequested.ExecuteIfBound();
		return FReply::Handled();
	}
	return SCompoundWidget::OnKeyDown(MyGeometry, InKeyEvent);
}

FReply SComputeGraphHlslEditor::OnTextBoxKeyDown(FGeometry const& MyGeometry, FKeyEvent const& InKeyEvent)
{
	if (InKeyEvent.GetKey() == EKeys::Tab && TextBox.IsValid())
	{
		// Insert spaces to the next 4-column tab stop so Tab doesn't navigate focus away.
		const int32 Column = TextBox->GetCursorLocation().GetOffset();
		const int32 NumSpaces = 4 - (Column % 4);
		TextBox->InsertTextAtCursor(FString::ChrN(NumSpaces, TEXT(' ')));
		return FReply::Handled();
	}
	return FReply::Unhandled();
}

#undef LOCTEXT_NAMESPACE
