// Copyright Epic Games, Inc. All Rights Reserved.

#include "ComputeFramework/ComputeGraphHlslSyntaxHighlighter.h"

#include "Framework/Text/IRun.h"
#include "Framework/Text/SlateTextRun.h"
#include "Framework/Text/TextLayout.h"
#include "Text/HLSLSyntaxTokenizer.h"

TSharedRef<FComputeGraphHlslSyntaxHighlighter> FComputeGraphHlslSyntaxHighlighter::Create(FSyntaxTextStyle const& InSyntaxTextStyle, FTextBlockStyle const& InBoundFunctionStyle)
{
	return MakeShareable(new FComputeGraphHlslSyntaxHighlighter(CreateTokenizer(), InSyntaxTextStyle, InBoundFunctionStyle));
}

FComputeGraphHlslSyntaxHighlighter::FComputeGraphHlslSyntaxHighlighter(
	TSharedPtr<ISyntaxTokenizer> InTokenizer,
	FSyntaxTextStyle const& InSyntaxTextStyle,
	FTextBlockStyle const& InBoundFunctionStyle)
	: FHLSLSyntaxHighlighterMarshaller(MoveTemp(InTokenizer), InSyntaxTextStyle)
	, BoundFunctionStyle(InBoundFunctionStyle)
{
}

void FComputeGraphHlslSyntaxHighlighter::SetBoundFunctions(TArray<FString> const& Names)
{
	BoundFunctionNames = TSet<FString>(Names);
	MakeDirty();
}

FTextLayout::FNewLineData FComputeGraphHlslSyntaxHighlighter::ProcessTokenizedLine(
	ISyntaxTokenizer::FTokenizedLine const& TokenizedLine,
	int32 const& LineNumber,
	FString const& SourceString,
	EParseState& ParseState)
{
	FTextLayout::FNewLineData LineData = FHLSLSyntaxHighlighterMarshaller::ProcessTokenizedLine(TokenizedLine, LineNumber, SourceString, ParseState);

	if (BoundFunctionNames.IsEmpty())
	{
		return LineData;
	}

	// Replace runs whose text matches a bound function name.
	for (TSharedRef<IRun>& Run : LineData.Runs)
	{
		const FTextRange Range = Run->GetTextRange();
		const FString Token = LineData.Text->Mid(Range.BeginIndex, Range.Len());
		if (BoundFunctionNames.Contains(Token))
		{
			Run = FSlateTextRun::Create(FRunInfo(TEXT("SyntaxHighlight.HLSL.BoundFunction")), LineData.Text, BoundFunctionStyle, Range);
		}
	}

	return LineData;
}
