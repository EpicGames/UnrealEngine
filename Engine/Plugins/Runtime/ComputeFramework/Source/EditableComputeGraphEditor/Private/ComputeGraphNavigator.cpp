// Copyright Epic Games, Inc. All Rights Reserved.

#include "ComputeFramework/ComputeGraphNavigator.h"

#include "ComputeFramework/EditableComputeGraph.h"
#include "Framework/Commands/UIAction.h"
#include "Input/Events.h"
#include "InputCoreTypes.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Styling/AppStyle.h"
#include "Styling/StarshipCoreStyle.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Text/SInlineEditableTextBlock.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Views/SExpanderArrow.h"
#include "Widgets/Views/STableRow.h"

#define LOCTEXT_NAMESPACE "ComputeFrameworkEditor"

/** Internal tree node for SComputeGraphNavigator. Either a section header or a leaf item. */
struct FNavigatorItem
{
	/** Display name shown in the row. */
	FName Name;
	/** Which kind of graph item this node represents. */
	EComputeGraphItemKind Kind = EComputeGraphItemKind::None;
	/** Index into the relevant array in FComputeGraphDesc. INDEX_NONE for header nodes. */
	int32 Index = INDEX_NONE;
	/** True for section header nodes, false for leaf item nodes. */
	bool bHeader = false;
	/** Child leaf items. Populated for header nodes only. */
	TArray<FNavigatorItemPtr> Children;

	static TSharedRef<FNavigatorItem> MakeHeader(FName InName, EComputeGraphItemKind InKind)
	{
		TSharedRef<FNavigatorItem> Item = MakeShared<FNavigatorItem>();
		Item->Name = InName;
		Item->Kind = InKind;
		Item->bHeader = true;
		return Item;
	}

	static TSharedRef<FNavigatorItem> MakeLeaf(FName InName, EComputeGraphItemKind InKind, int32 InIndex)
	{
		TSharedRef<FNavigatorItem> Item = MakeShared<FNavigatorItem>();
		Item->Name = InName;
		Item->Kind = InKind;
		Item->Index = InIndex;
		return Item;
	}
};

/** Tree row for a section header. Displays the section name, an expander arrow, and an Add button. */
class SNavigatorHeaderRow : public STableRow<FNavigatorItemPtr>
{
public:
	SLATE_BEGIN_ARGS(SNavigatorHeaderRow) {}
		SLATE_ARGUMENT(FNavigatorItemPtr, Item)
		SLATE_EVENT(FOnComputeGraphAddItem, OnAddItem)
	SLATE_END_ARGS()

	void Construct(FArguments const& InArgs, TSharedRef<STableViewBase> const& InOwnerTable)
	{
		ItemPtr = InArgs._Item;
		OnAddItem = InArgs._OnAddItem;

		STableRow<FNavigatorItemPtr>::ConstructInternal(
			STableRow<FNavigatorItemPtr>::FArguments()
			.Style(FAppStyle::Get(), "DetailsView.TreeView.TableRow")
			.ShowSelection(false),
			InOwnerTable);

		ChildSlot
		.Padding(0.f, 2.f, 0.f, 0.f)
		[
			SNew(SBorder)
			.BorderImage(this, &SNavigatorHeaderRow::GetBorderBrush)
			.Padding(FMargin(3.f, 5.f))
			[
				SNew(SHorizontalBox)

				// Expander arrow
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				[
					SNew(SExpanderArrow, SharedThis(this))
					.IndentAmount(8)
					.ShouldDrawWires(false)
				]

				// Section title
				+ SHorizontalBox::Slot()
				.FillWidth(1.f)
				.VAlign(VAlign_Center)
				.Padding(2.f, 0.f)
				[
					SNew(STextBlock)
					.Text(FText::FromName(ItemPtr->Name))
					.TextStyle(FAppStyle::Get(), "DetailsView.CategoryTextStyle")
					.Font(FStyleFonts::Get().NormalBold)
				]

				// Add button
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				[
					SNew(SButton)
					.ButtonStyle(FAppStyle::Get(), "SimpleButton")
					.ContentPadding(FMargin(4.f, 0.f))
					.ToolTipText(LOCTEXT("Navigator_AddItemTooltip", "Add new item"))
					.OnClicked_Lambda([this]()
					{
						OnAddItem.ExecuteIfBound(ItemPtr->Kind);
						return FReply::Handled();
					})
					[
						SNew(SImage)
						.Image(FAppStyle::Get().GetBrush("Icons.PlusCircle"))
						.ColorAndOpacity(FSlateColor::UseForeground())
					]
				]
			]
		];
	}

	/** Clicking anywhere on the header bar toggles expansion. */
	FReply OnMouseButtonDown(FGeometry const& MyGeometry, FPointerEvent const& MouseEvent) override
	{
		if (MouseEvent.GetEffectingButton() == EKeys::LeftMouseButton)
		{
			ToggleExpansion();
			return FReply::Handled();
		}
		return STableRow<FNavigatorItemPtr>::OnMouseButtonDown(MyGeometry, MouseEvent);
	}

private:
	FSlateBrush const* GetBorderBrush() const
	{
		return IsHovered() ? FAppStyle::Get().GetBrush("Brushes.Secondary") : FAppStyle::Get().GetBrush("Brushes.Header");
	}

	/** The header item this row represents. */
	FNavigatorItemPtr ItemPtr;
	/** Delegate fired when the user clicks the Add button. */
	FOnComputeGraphAddItem OnAddItem;
};

/** Tree row for a leaf item. Displays the item name with inline rename support. */
class SNavigatorLeafRow : public STableRow<FNavigatorItemPtr>
{
public:
	SLATE_BEGIN_ARGS(SNavigatorLeafRow) {}
		SLATE_ARGUMENT(FNavigatorItemPtr, Item)
		SLATE_EVENT(FOnComputeGraphRenameItem, OnRenameItem)
		SLATE_EVENT(FOnVerifyTextChanged, OnVerifyRenameText)
	SLATE_END_ARGS()

	void Construct(FArguments const& InArgs, TSharedRef<STableViewBase> const& InOwnerTable)
	{
		ItemPtr = InArgs._Item;
		OnRenameItem = InArgs._OnRenameItem;

		STableRow<FNavigatorItemPtr>::Construct(
			STableRow<FNavigatorItemPtr>::FArguments()
			.Padding(FMargin(20.f, 1.f, 4.f, 1.f))
			.Content()
			[
				SAssignNew(InlineText, SInlineEditableTextBlock)
				.Text(FText::FromName(ItemPtr->Name))
				.Font(FAppStyle::Get().GetFontStyle("NormalFont"))
				.IsSelected(this, &SNavigatorLeafRow::IsSelectedExclusively)
				.OnVerifyTextChanged(InArgs._OnVerifyRenameText)
				.OnTextCommitted(this, &SNavigatorLeafRow::OnTextCommitted)
			],
			InOwnerTable);
	}

	void EnterRenameMode()
	{
		if (InlineText.IsValid())
		{
			InlineText->EnterEditingMode();
		}
	}

private:
	void OnTextCommitted(FText const& NewText, ETextCommit::Type CommitType)
	{
		const FString Trimmed = NewText.ToString().TrimStartAndEnd();
		if (!Trimmed.IsEmpty() && CommitType != ETextCommit::OnCleared)
		{
			OnRenameItem.ExecuteIfBound(ItemPtr->Kind, ItemPtr->Index, FName(*Trimmed));
		}
	}

	/** The leaf item this row represents. */
	FNavigatorItemPtr ItemPtr;
	/** The editable text block used for inline renaming. */
	TSharedPtr<SInlineEditableTextBlock> InlineText;
	/** Delegate fired when the user commits a rename. */
	FOnComputeGraphRenameItem OnRenameItem;
};

void SComputeGraphNavigator::Construct(FArguments const& InArgs)
{
	Asset = InArgs._Asset;
	OnItemSelected = InArgs._OnItemSelected;
	OnAddItem = InArgs._OnAddItem;
	OnDeleteItem = InArgs._OnDeleteItem;
	OnDuplicateItem = InArgs._OnDuplicateItem;
	OnRenameItem = InArgs._OnRenameItem;
	OnKernelSelected = InArgs._OnKernelSelected;

	RebuildItems();

	ChildSlot
	[
		SNew(SBorder)
		.BorderImage(FAppStyle::Get().GetBrush("ToolPanel.GroupBorder"))
		.Padding(0.f)
		[
			SAssignNew(TreeView, STreeView<FNavigatorItemPtr>)
			.TreeItemsSource(&RootItems)
			.OnGenerateRow(this, &SComputeGraphNavigator::GenerateRow)
			.OnGetChildren(this, &SComputeGraphNavigator::OnGetChildren)
			.OnSelectionChanged(this, &SComputeGraphNavigator::OnSelectionChanged)
			.OnExpansionChanged(this, &SComputeGraphNavigator::OnExpansionChanged)
			.OnContextMenuOpening(this, &SComputeGraphNavigator::BuildContextMenu)
			.SelectionMode(ESelectionMode::Single)
		]
	];

	// Expand all sections by default.
	for (FNavigatorItemPtr const& Root : RootItems)
	{
		TreeView->SetItemExpansion(Root, true);
	}
}

void SComputeGraphNavigator::Refresh()
{
	RebuildItems();
	if (!TreeView.IsValid()) return;

	TreeView->RequestTreeRefresh();

	// Restore expansion state.
	// Expand unless the section was manually collapsed.
	for (FNavigatorItemPtr const& Root : RootItems)
	{
		TreeView->SetItemExpansion(Root, !CollapsedSections.Contains(Root->Kind));
	}

	// Re-apply selection by matching kind + index in the new leaf items.
	if (Selection.Kind != EComputeGraphItemKind::None && Selection.Index != INDEX_NONE)
	{
		for (FNavigatorItemPtr const& Root : RootItems)
		{
			for (FNavigatorItemPtr const& Child : Root->Children)
			{
				if (Child->Kind == Selection.Kind && Child->Index == Selection.Index)
				{
					TreeView->SetSelection(Child, ESelectInfo::Direct);
					return;
				}
			}
		}
	}
}

void SComputeGraphNavigator::SetSelectedItem(EComputeGraphItemKind Kind, int32 Index)
{
	Selection.Kind = Kind;
	Selection.Index = Index;

	if (!TreeView.IsValid()) return;

	if (Kind == EComputeGraphItemKind::None || Index == INDEX_NONE)
	{
		TreeView->ClearSelection();
		return;
	}

	for (FNavigatorItemPtr const& Root : RootItems)
	{
		for (FNavigatorItemPtr const& Child : Root->Children)
		{
			if (Child->Kind == Kind && Child->Index == Index)
			{
				TreeView->SetSelection(Child, ESelectInfo::Direct);
				return;
			}
		}
	}
}

void SComputeGraphNavigator::RebuildItems()
{
	RootItems.Reset();
	if (!Asset.IsValid()) return;

	FComputeGraphDesc const& Desc = Asset->GetGraphDescription();

	// Binding Objects.
	TSharedRef<FNavigatorItem> BindingSection = FNavigatorItem::MakeHeader(TEXT("Binding Objects"), EComputeGraphItemKind::BindingObject);
	for (int32 Index = 0; Index < Desc.BindingObjects.Num(); ++Index)
	{
		const FName N = Desc.BindingObjects[Index].Name;
		BindingSection->Children.Add(FNavigatorItem::MakeLeaf(N.IsNone() ? FName(TEXT("(unnamed)")) : N, EComputeGraphItemKind::BindingObject, Index));
	}
	RootItems.Add(BindingSection);

	// Data Interfaces.
	TSharedRef<FNavigatorItem> InterfaceSection = FNavigatorItem::MakeHeader(TEXT("Data Interfaces"), EComputeGraphItemKind::Interface);
	for (int32 Index = 0; Index < Desc.DataInterfaces.Num(); ++Index)
	{
		const FName N = Desc.DataInterfaces[Index].Name;
		InterfaceSection->Children.Add(FNavigatorItem::MakeLeaf(N.IsNone() ? FName(TEXT("(unnamed)")) : N, EComputeGraphItemKind::Interface, Index));
	}
	RootItems.Add(InterfaceSection);

	// Kernels.
	TSharedRef<FNavigatorItem> KernelSection = FNavigatorItem::MakeHeader(TEXT("Kernels"), EComputeGraphItemKind::Kernel);
	for (int32 Index = 0; Index < Desc.Kernels.Num(); ++Index)
	{
		const FName N = Desc.Kernels[Index].Name;
		KernelSection->Children.Add(FNavigatorItem::MakeLeaf(N.IsNone() ? FName(TEXT("(unnamed)")) : N, EComputeGraphItemKind::Kernel, Index));
	}
	RootItems.Add(KernelSection);
}

TSharedRef<ITableRow> SComputeGraphNavigator::GenerateRow(
	FNavigatorItemPtr Item,
	TSharedRef<STableViewBase> const& OwnerTable)
{
	if (Item->bHeader)
	{
		return SNew(SNavigatorHeaderRow, OwnerTable)
			.Item(Item)
			.OnAddItem(FOnComputeGraphAddItem::CreateSP(this, &SComputeGraphNavigator::OnAddItemClicked));
	}

	const EComputeGraphItemKind Kind = Item->Kind;
	const int32 Index = Item->Index;
	return SNew(SNavigatorLeafRow, OwnerTable)
		.Item(Item)
		.OnRenameItem(FOnComputeGraphRenameItem::CreateSP(this, &SComputeGraphNavigator::OnRenameItemCommitted))
		.OnVerifyRenameText(FOnVerifyTextChanged::CreateSP(this, &SComputeGraphNavigator::VerifyRenameText, Kind, Index));
}

void SComputeGraphNavigator::OnGetChildren(FNavigatorItemPtr Item, TArray<FNavigatorItemPtr>& OutChildren)
{
	OutChildren.Append(Item->Children);
}

TSharedPtr<SWidget> SComputeGraphNavigator::BuildContextMenu()
{
	TArray<FNavigatorItemPtr> const SelectedItems = TreeView->GetSelectedItems();
	if (SelectedItems.IsEmpty() || SelectedItems[0]->bHeader)
	{
		return nullptr;
	}

	FMenuBuilder MenuBuilder(/*bInShouldCloseWindowAfterMenuSelection=*/true, nullptr);

	MenuBuilder.AddMenuEntry(
		LOCTEXT("Navigator_DeleteItem", "Delete"),
		LOCTEXT("Navigator_DeleteItemTooltip", "Remove this item from the graph description."),
		FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icons.Delete"),
		FUIAction(FExecuteAction::CreateSP(this, &SComputeGraphNavigator::OnDeleteSelected)));

	MenuBuilder.AddMenuEntry(
		LOCTEXT("Navigator_DuplicateItem", "Duplicate"),
		LOCTEXT("Navigator_DuplicateItemTooltip", "Add a copy of this item with a new name."),
		FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icons.Duplicate"),
		FUIAction(FExecuteAction::CreateSP(this, &SComputeGraphNavigator::OnDuplicateSelected)));

	MenuBuilder.AddMenuEntry(
		LOCTEXT("Navigator_RenameItem", "Rename"),
		LOCTEXT("Navigator_RenameItemTooltip", "Rename this item inline."),
		FSlateIcon(FAppStyle::GetAppStyleSetName(), "GenericCommands.Rename"),
		FUIAction(FExecuteAction::CreateSP(this, &SComputeGraphNavigator::OnRenameSelected)));

	return MenuBuilder.MakeWidget();
}

void SComputeGraphNavigator::OnSelectionChanged(FNavigatorItemPtr Item, ESelectInfo::Type SelectInfo)
{
	if (!Item.IsValid() || Item->bHeader)
	{
		Selection.Kind = EComputeGraphItemKind::None;
		Selection.Index = INDEX_NONE;
		OnItemSelected.ExecuteIfBound(EComputeGraphItemKind::None, INDEX_NONE, NAME_None);
		return;
	}

	Selection.Kind = Item->Kind;
	Selection.Index = Item->Index;
	OnItemSelected.ExecuteIfBound(Item->Kind, Item->Index, Item->Name);

	if (Item->Kind == EComputeGraphItemKind::Kernel)
	{
		OnKernelSelected.ExecuteIfBound(Item->Name);
	}
}

void SComputeGraphNavigator::OnExpansionChanged(FNavigatorItemPtr Item, bool bExpanded)
{
	if (!Item.IsValid() || !Item->bHeader) return;

	if (bExpanded)
	{
		CollapsedSections.Remove(Item->Kind);
	}
	else
	{
		CollapsedSections.Add(Item->Kind);
	}
}

void SComputeGraphNavigator::OnAddItemClicked(EComputeGraphItemKind Kind)
{
	OnAddItem.ExecuteIfBound(Kind);
}

void SComputeGraphNavigator::OnDeleteSelected()
{
	if (Selection.Kind != EComputeGraphItemKind::None && Selection.Index != INDEX_NONE)
	{
		OnDeleteItem.ExecuteIfBound(Selection.Kind, Selection.Index);
	}
}

void SComputeGraphNavigator::OnDuplicateSelected()
{
	if (Selection.Kind != EComputeGraphItemKind::None && Selection.Index != INDEX_NONE)
	{
		OnDuplicateItem.ExecuteIfBound(Selection.Kind, Selection.Index);
	}
}

void SComputeGraphNavigator::OnRenameItemCommitted(EComputeGraphItemKind Kind, int32 Index, FName NewName)
{
	OnRenameItem.ExecuteIfBound(Kind, Index, NewName);
}

bool SComputeGraphNavigator::VerifyRenameText(FText const& NewText, FText& OutError, EComputeGraphItemKind Kind, int32 Index) const
{
	const FString Trimmed = NewText.ToString().TrimStartAndEnd();
	if (Trimmed.IsEmpty())
	{
		OutError = LOCTEXT("Navigator_RenameEmpty", "Name cannot be empty.");
		return false;
	}

	const FName Candidate(*Trimmed);

	for (FNavigatorItemPtr const& Root : RootItems)
	{
		for (FNavigatorItemPtr const& Child : Root->Children)
		{
			if (Child->Kind == Kind && Child->Index != Index && Child->Name == Candidate)
			{
				OutError = FText::Format(LOCTEXT("Navigator_RenameDuplicate", "'{0}' is already in use."), NewText);
				return false;
			}
		}
	}
	return true;
}

FReply SComputeGraphNavigator::OnKeyDown(FGeometry const& MyGeometry, FKeyEvent const& InKeyEvent)
{
	if (InKeyEvent.GetKey() == EKeys::F2)
	{
		OnRenameSelected();
		return FReply::Handled();
	}
	return SCompoundWidget::OnKeyDown(MyGeometry, InKeyEvent);
}

void SComputeGraphNavigator::OnRenameSelected()
{
	if (Selection.Kind == EComputeGraphItemKind::None || Selection.Index == INDEX_NONE) 
	{
		return;
	}

	for (FNavigatorItemPtr const& Root : RootItems)
	{
		for (FNavigatorItemPtr const& Child : Root->Children)
		{
			if (Child->Kind == Selection.Kind && Child->Index == Selection.Index)
			{
				TSharedPtr<ITableRow> Row = TreeView->WidgetFromItem(Child);
				if (TSharedPtr<SNavigatorLeafRow> LeafRow = StaticCastSharedPtr<SNavigatorLeafRow>(Row))
				{
					LeafRow->EnterRenameMode();
				}
				return;
			}
		}
	}
}

#undef LOCTEXT_NAMESPACE
