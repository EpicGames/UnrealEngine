// Copyright Epic Games, Inc. All Rights Reserved.

#include "ComputeFramework/ComputeGraphView.h"

#include "ComputeFramework/EditableComputeGraph.h"
#include "Input/Events.h"
#include "Rendering/DrawElements.h"
#include "Styling/AppStyle.h"
#include "Styling/SlateBrush.h"
#include "Widgets/Layout/SBorder.h"

#define LOCTEXT_NAMESPACE "ComputeFrameworkEditor"

namespace
{
	// Layout constants (local pixels, unscaled by DPI).
	constexpr float NodeWidth = 160.f;
	constexpr float NodeHeight = 36.f;
	constexpr float NodeVGap = 12.f;
	constexpr float ColXStart = 20.f;
	constexpr float ColStep = NodeWidth + 60.f; // 220 px per rank column
	constexpr float TopPadding = 20.f;

	constexpr float ViewMinWidth = ColXStart + ColStep + 20.f;
	constexpr float ViewMinHeight = 200.f;

	// Colours.
	static const FLinearColor ColNode(0.15f, 0.15f, 0.18f, 1.f);
	static const FLinearColor ColNodeBorder(0.40f, 0.40f, 0.45f, 1.f);
	static const FLinearColor ColNodeBorderSelected(1.00f, 0.70f, 0.10f, 1.f);
	static const FLinearColor ColNodeHighlightKernel(0.20f, 0.35f, 0.45f, 1.f);
	static const FLinearColor ColNodeHighlightInterface(0.35f, 0.20f, 0.45f, 1.f);
	static const FLinearColor ColNodeBorderHighlightKernel(0.50f, 0.80f, 1.00f, 1.f);
	static const FLinearColor ColNodeBorderHighlightInterface(0.85f, 0.50f, 1.00f, 1.f);
	static const FLinearColor ColEdge(0.55f, 0.75f, 0.55f, 0.8f);
	static const FLinearColor ColEdgeOrphaned(0.90f, 0.20f, 0.20f, 0.8f);
	static const FLinearColor ColText(0.90f, 0.90f, 0.90f, 1.f);

} // namespace

void SComputeGraphView::Construct(FArguments const& InArgs)
{
	Asset = InArgs._Asset;
	OnNodeClicked = InArgs._OnNodeClicked;
	RebuildLayout();
}

void SComputeGraphView::SetHighlightedKernel(FName KernelName)
{
	HighlightedKernel = KernelName;
	Invalidate(EInvalidateWidgetReason::Paint);
}

void SComputeGraphView::SetHighlightedInterface(FName InterfaceName)
{
	HighlightedInterface = InterfaceName;
	Invalidate(EInvalidateWidgetReason::Paint);
}

void SComputeGraphView::Refresh()
{
	RebuildLayout();
	Invalidate(EInvalidateWidgetReason::Layout | EInvalidateWidgetReason::Paint);
}

void SComputeGraphView::RebuildLayout()
{
	NodeLayouts.Reset();
	Edges.Reset();
	InterfaceCentreMap.Reset();
	KernelCentreMap.Reset();
	MaxRank = 0;

	if (!Asset.IsValid())
	{
		return;
	}

	FComputeGraphDesc const& Desc = Asset->GetGraphDescription();

	// Step 1: Rank assignment via longest-path layering
	// Rank = length of the longest path from any source node.
	// Sources are interfaces with no incoming kernel-output edges (rank 0).

	TMap<FName, int32> InterfaceRank;
	TMap<FName, int32> KernelRank;

	for (FComputeGraphDataInterfaceDesc const& DataInterfaceDesc : Desc.DataInterfaces)
	{
		if (!DataInterfaceDesc.Name.IsNone())
		{
			InterfaceRank.Add(DataInterfaceDesc.Name, 0);
		}
	}
	for (FComputeGraphKernelDesc const& KernelDesc : Desc.Kernels)
	{
		if (!KernelDesc.Name.IsNone())
		{
			KernelRank.Add(KernelDesc.Name, 0);
		}
	}

	// Which kernels write to each interface?
	TMap<FName, TArray<FName>> InterfaceWriters;
	for (FComputeGraphKernelDesc const& KernelDesc : Desc.Kernels)
	{
		for (FKernelPin const& Pin : KernelDesc.Outputs)
		{
			if (!Pin.DataInterfaceName.IsNone())
			{
				InterfaceWriters.FindOrAdd(Pin.DataInterfaceName).AddUnique(KernelDesc.Name);
			}
		}
	}

	// Propagate ranks until stable. Converges for DAGs in at most N+M iterations.
	// Cap iterations to prevent infinite loops on cyclic graphs.
	const int32 MaxIterations = Desc.DataInterfaces.Num() + Desc.Kernels.Num() + 1;
	bool bChanged = true;
	for (int32 Iteration = 0; bChanged && Iteration < MaxIterations; ++Iteration)
	{
		bChanged = false;

		// Kernel rank = max input interface rank + 1.
		for (FComputeGraphKernelDesc const& KernelDesc : Desc.Kernels)
		{
			if (KernelDesc.Name.IsNone())
			{
				continue;
			}
			int32 MaxInputRank = -1;
			for (FKernelPin const& Pin : KernelDesc.Inputs)
			{
				if (int32* InterfaceRankPtr = InterfaceRank.Find(Pin.DataInterfaceName))
				{
					MaxInputRank = FMath::Max(MaxInputRank, *InterfaceRankPtr);
				}
			}
			int32& CachedKernelRank = KernelRank.FindChecked(KernelDesc.Name);
			const int32 NewKernelRank = MaxInputRank + 1;
			if (NewKernelRank > CachedKernelRank)
			{
				CachedKernelRank = NewKernelRank;
				bChanged = true;
			}
		}

		// Interface rank = max writer-kernel rank + 1 (unchanged if no writers).
		for (FComputeGraphDataInterfaceDesc const& DataInterfaceDesc : Desc.DataInterfaces)
		{
			if (DataInterfaceDesc.Name.IsNone())
			{
				continue;
			}
			TArray<FName> const* Writers = InterfaceWriters.Find(DataInterfaceDesc.Name);
			if (!Writers)
			{
				continue;
			}
			int32 MaxWriterKernelRank = 0;
			for (FName const& WriterKernelName : *Writers)
			{
				if (int32* KernelRankPtr = KernelRank.Find(WriterKernelName))
				{
					MaxWriterKernelRank = FMath::Max(MaxWriterKernelRank, *KernelRankPtr);
				}
			}
			int32& CachedInterfaceRank = InterfaceRank.FindChecked(DataInterfaceDesc.Name);
			const int32 NewInterfaceRank = MaxWriterKernelRank + 1;
			if (NewInterfaceRank > CachedInterfaceRank)
			{
				CachedInterfaceRank = NewInterfaceRank;
				bChanged = true;
			}
		}
	}

	for (TPair<FName, int32> const& RankPair : InterfaceRank)
	{
		MaxRank = FMath::Max(MaxRank, RankPair.Value);
	}
	for (TPair<FName, int32> const& RankPair : KernelRank)
	{
		MaxRank = FMath::Max(MaxRank, RankPair.Value);
	}

	// Step 2: Place nodes, stacking vertically within each rank column.

	TMap<int32, int32> NodesPerRank;

	auto PlaceNode = [&](FName Name, bool bKernel, int32 Rank)
	{
		int32& Count = NodesPerRank.FindOrAdd(Rank);
		FNodeLayout& NewLayout = NodeLayouts.AddDefaulted_GetRef();
		NewLayout.Name = Name;
		NewLayout.Centre = FVector2D(
			ColXStart + Rank * ColStep + NodeWidth * 0.5f,
			TopPadding + Count * (NodeHeight + NodeVGap) + NodeHeight * 0.5f);
		NewLayout.bKernel = bKernel;
		NewLayout.Rank = Rank;
		++Count;
	};

	// Maintain declaration order within each rank.
	for (FComputeGraphDataInterfaceDesc const& DataInterfaceDesc : Desc.DataInterfaces)
	{
		if (!DataInterfaceDesc.Name.IsNone())
		{
			if (int32* RankPtr = InterfaceRank.Find(DataInterfaceDesc.Name))
			{
				PlaceNode(DataInterfaceDesc.Name, false, *RankPtr);
			}
		}
	}

	for (FComputeGraphKernelDesc const& KernelDesc : Desc.Kernels)
	{
		if (!KernelDesc.Name.IsNone())
		{
			if (int32* RankPtr = KernelRank.Find(KernelDesc.Name))
			{
				PlaceNode(KernelDesc.Name, true, *RankPtr);
			}
		}
	}

	// Step 3: Build edges.

	auto AddEdges = [&](FComputeGraphKernelDesc const& KernelDesc, TArray<FKernelPin> const& Pins, bool bIsOutput)
	{
		for (FKernelPin const& Pin : Pins)
		{
			FEdge& NewEdge = Edges.AddDefaulted_GetRef();
			NewEdge.KernelName = KernelDesc.Name;
			NewEdge.InterfaceName = Pin.DataInterfaceName;
			NewEdge.KernelFn = Pin.KernelFunctionName;
			NewEdge.InterfaceFn = Pin.DataInterfaceFunctionName;
			NewEdge.bIsOutput = bIsOutput;
			NewEdge.bOrphaned = Pin.bOrphaned || Pin.DataInterfaceName.IsNone() || Pin.DataInterfaceFunctionName.IsEmpty();
		}
	};
	for (FComputeGraphKernelDesc const& KernelDesc : Desc.Kernels)
	{
		AddEdges(KernelDesc, KernelDesc.Inputs, false);
		AddEdges(KernelDesc, KernelDesc.Outputs, true);
	}

	// Cache centre maps for OnPaint (avoids per-frame heap allocation).
	for (FNodeLayout const& NodeLayout : NodeLayouts)
	{
		(NodeLayout.bKernel ? KernelCentreMap : InterfaceCentreMap).Add(NodeLayout.Name, NodeLayout.Centre);
	}

	// Cache height metric for ComputeDesiredSize (NodesPerRank is fully populated above).
	MaxNodesInAnyRank = 0;
	for (TPair<int32, int32> const& RankPair : NodesPerRank)
	{
		MaxNodesInAnyRank = FMath::Max(MaxNodesInAnyRank, RankPair.Value);
	}
}

static void DrawNode(
	FSlateWindowElementList& OutDrawElements,
	int32 LayerId,
	FGeometry const& AllottedGeometry,
	FVector2D Centre,
	FName Name,
	FSlateFontInfo const& Font,
	FLinearColor Fill,
	FLinearColor Border,
	float BorderThickness)
{
	const FVector2D TopLeft = Centre - FVector2D(NodeWidth * 0.5f, NodeHeight * 0.5f);

	// For thick (highlighted) borders draw an oversized filled rect behind the node.
	if (BorderThickness > 1.f)
	{
		const FVector2D Expand(BorderThickness, BorderThickness);
		FSlateDrawElement::MakeBox(
			OutDrawElements,
			LayerId,
			AllottedGeometry.ToPaintGeometry(FVector2D(NodeWidth, NodeHeight) + Expand * 2.f, FSlateLayoutTransform(TopLeft - Expand)),
			FAppStyle::GetBrush("WhiteBrush"),
			ESlateDrawEffect::None,
			Border);
	}

	// Main node rect.
	FSlateDrawElement::MakeBox(
		OutDrawElements,
		LayerId + 1,
		AllottedGeometry.ToPaintGeometry(FVector2D(NodeWidth, NodeHeight), FSlateLayoutTransform(TopLeft)),
		FAppStyle::GetBrush("WhiteBrush"),
		ESlateDrawEffect::None,
		Fill);

	// Thin border (non-highlighted). 
	if (BorderThickness <= 1.f)
	{
		FSlateDrawElement::MakeBox(
			OutDrawElements,
			LayerId + 1,
			AllottedGeometry.ToPaintGeometry(FVector2D(NodeWidth, NodeHeight), FSlateLayoutTransform(TopLeft)),
			FAppStyle::GetBrush("Border"),
			ESlateDrawEffect::None,
			Border);
	}

	// Node name text.
	FSlateDrawElement::MakeText(
		OutDrawElements,
		LayerId + 2,
		AllottedGeometry.ToPaintGeometry(
			FVector2D(NodeWidth - 8.f, NodeHeight),
			FSlateLayoutTransform(TopLeft + FVector2D(4.f, (NodeHeight - Font.Size) * 0.5f))),
		Name.ToString(),
		Font,
		ESlateDrawEffect::None,
		ColText);
}

int32 SComputeGraphView::OnPaint(
	FPaintArgs const& Args,
	FGeometry const& AllottedGeometry,
	FSlateRect const& MyCullingRect,
	FSlateWindowElementList& OutDrawElements,
	int32 LayerId,
	FWidgetStyle const& InWidgetStyle,
	bool bParentEnabled) const
{
	// Dark background.
	FSlateDrawElement::MakeBox(
		OutDrawElements,
		LayerId,
		AllottedGeometry.ToPaintGeometry(),
		FAppStyle::GetBrush("WhiteBrush"),
		ESlateDrawEffect::None,
		FLinearColor(0.07f, 0.07f, 0.07f, 1.f));

	int32 CurrentLayer = LayerId + 1;

	const FSlateFontInfo Font = FAppStyle::Get().GetFontStyle("PropertyWindow.NormalFont");

	// Draw edges first (below nodes).
	for (FEdge const& Edge : Edges)
	{
		FVector2D const* KernelCentre = KernelCentreMap.Find(Edge.KernelName);
		FVector2D const* InterfaceCentre = InterfaceCentreMap.Find(Edge.InterfaceName);
		if (!KernelCentre || !InterfaceCentre)
		{
			continue;
		}

		FVector2D P0, P3;
		if (Edge.bIsOutput)
		{
			// Output: right side of kernel to left side of (higher-rank) interface.
			P0 = *KernelCentre + FVector2D(NodeWidth * 0.5f, 0.f);
			P3 = *InterfaceCentre - FVector2D(NodeWidth * 0.5f, 0.f);
		}
		else
		{
			// Input: right side of interface to left side of (higher-rank) kernel.
			P0 = *InterfaceCentre + FVector2D(NodeWidth * 0.5f, 0.f);
			P3 = *KernelCentre - FVector2D(NodeWidth * 0.5f, 0.f);
		}

		const float TanX = (P3.X - P0.X) * 0.4f;
		const FVector2D P1(P0.X + TanX, P0.Y);
		const FVector2D P2(P3.X - TanX, P3.Y);

		FSlateDrawElement::MakeCubicBezierSpline(
			OutDrawElements,
			CurrentLayer,
			AllottedGeometry.ToPaintGeometry(),
			P0, P1, P2, P3,
			2.f,
			ESlateDrawEffect::None,
			Edge.bOrphaned ? ColEdgeOrphaned : ColEdge);
	}
	++CurrentLayer;

	// Collect nodes adjacent to the primary selection (connected via an edge).
	TSet<FName> AdjacentInterfaces;
	TSet<FName> AdjacentKernels;

	if (!HighlightedKernel.IsNone())
	{
		for (FEdge const& Edge : Edges)
		{
			if (Edge.KernelName == HighlightedKernel && !Edge.InterfaceName.IsNone())
			{
				AdjacentInterfaces.Add(Edge.InterfaceName);
			}
		}
	}
	if (!HighlightedInterface.IsNone())
	{
		for (FEdge const& Edge : Edges)
		{
			if (Edge.InterfaceName == HighlightedInterface && !Edge.KernelName.IsNone())
			{
				AdjacentKernels.Add(Edge.KernelName);
			}
		}
	}

	// Draw nodes.
	for (FNodeLayout const& NodeLayout : NodeLayouts)
	{
		const bool bIsSelected = (NodeLayout.bKernel && NodeLayout.Name == HighlightedKernel) || (!NodeLayout.bKernel && NodeLayout.Name == HighlightedInterface);
		const bool bIsAdjacent = !bIsSelected && (NodeLayout.bKernel ? AdjacentKernels.Contains(NodeLayout.Name) : AdjacentInterfaces.Contains(NodeLayout.Name));

		const FLinearColor Fill = (bIsSelected || bIsAdjacent)
		                        ? (NodeLayout.bKernel ? ColNodeHighlightKernel : ColNodeHighlightInterface)
		                        : ColNode;
		const FLinearColor Border = bIsSelected
		                          ? ColNodeBorderSelected
		                          : bIsAdjacent ? (NodeLayout.bKernel ? ColNodeBorderHighlightKernel : ColNodeBorderHighlightInterface)
		                          : ColNodeBorder;

		DrawNode(OutDrawElements, CurrentLayer, AllottedGeometry, NodeLayout.Centre, NodeLayout.Name, Font, Fill, Border, bIsSelected ? 3.f : 1.f);
	}
	CurrentLayer += 3;

	return CurrentLayer;
}

FVector2D SComputeGraphView::ComputeDesiredSize(float LayoutScaleMultiplier) const
{
	const float Width = FMath::Max(ViewMinWidth, ColXStart + (MaxRank + 1) * ColStep + 20.f);
	const float Height = FMath::Max(ViewMinHeight, TopPadding + MaxNodesInAnyRank * (NodeHeight + NodeVGap));
	return FVector2D(Width, Height);
}

FReply SComputeGraphView::OnMouseButtonDown(FGeometry const& MyGeometry, FPointerEvent const& MouseEvent)
{
	if (MouseEvent.GetEffectingButton() != EKeys::LeftMouseButton || !OnNodeClicked.IsBound())
	{
		return FReply::Unhandled();
	}

	const FVector2D LocalPos = MyGeometry.AbsoluteToLocal(MouseEvent.GetScreenSpacePosition());

	for (FNodeLayout const& NodeLayout : NodeLayouts)
	{
		if (FMath::Abs(LocalPos.X - NodeLayout.Centre.X) <= NodeWidth * 0.5f &&
		    FMath::Abs(LocalPos.Y - NodeLayout.Centre.Y) <= NodeHeight * 0.5f)
		{
			OnNodeClicked.Execute(NodeLayout.Name, NodeLayout.bKernel);
			return FReply::Handled();
		}
	}

	return FReply::Unhandled();
}

#undef LOCTEXT_NAMESPACE
