// Copyright Epic Games, Inc. All Rights Reserved.

#include "ComputeFramework/EditableComputeGraphAssetDefinition.h"

#include "ComputeFramework/EditableComputeGraph.h"
#include "ComputeFramework/EditableComputeGraphEditorToolkit.h"

FText UAssetDefinition_EditableComputeGraph::GetAssetDisplayName() const
{
	return NSLOCTEXT("ComputeFramework", "EditableComputeGraphName", "Compute Graph (Experimental)");
}

FLinearColor UAssetDefinition_EditableComputeGraph::GetAssetColor() const
{
	return FColor::Orange;
}

TSoftClassPtr<UObject> UAssetDefinition_EditableComputeGraph::GetAssetClass() const
{
	return UEditableComputeGraph::StaticClass();
}

TConstArrayView<FAssetCategoryPath> UAssetDefinition_EditableComputeGraph::GetAssetCategories() const
{
	static const auto Categories = { EAssetCategoryPaths::Misc };
	return Categories;
}

EAssetCommandResult UAssetDefinition_EditableComputeGraph::OpenAssets(const FAssetOpenArgs& OpenArgs) const
{
	for (UEditableComputeGraph* EditableComputeGraph : OpenArgs.LoadObjects<UEditableComputeGraph>())
	{
		FEditableComputeGraphEditorToolkit::Create(EditableComputeGraph, OpenArgs.GetToolkitMode(), OpenArgs.ToolkitHost);
	}
	return EAssetCommandResult::Handled;
}
