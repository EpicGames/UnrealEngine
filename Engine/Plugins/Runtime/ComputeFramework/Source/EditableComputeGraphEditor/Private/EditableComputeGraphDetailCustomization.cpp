// Copyright Epic Games, Inc. All Rights Reserved.

#include "ComputeFramework/EditableComputeGraphDetailCustomization.h"

#include "ComputeFramework/ComputeGraphNavigator.h"
#include "ComputeFramework/EditableComputeGraph.h"
#include "ComputeFramework/HlslExternalPinParser.h"
#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "PropertyHandle.h"
#include "Styling/AppStyle.h"
#include "UObject/UnrealType.h"
#include "Widgets/Input/SComboBox.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "ComputeFrameworkEditor"

TSharedRef<IDetailCustomization> FEditableComputeGraphDetailCustomization::MakeInstance(TSharedPtr<FComputeGraphEditorSelection> InSelection)
{
	TSharedRef<FEditableComputeGraphDetailCustomization> Instance = MakeShared<FEditableComputeGraphDetailCustomization>();
	Instance->Selection = MoveTemp(InSelection);
	return Instance;
}

void FEditableComputeGraphDetailCustomization::CustomizeDetails(IDetailLayoutBuilder& DetailBuilder)
{
	// Always hide the raw asset-level categories. We manage display ourselves.
	DetailBuilder.HideCategory(TEXT("Graph"));
	DetailBuilder.HideCategory(TEXT("Kernel"));
	DetailBuilder.HideCategory(TEXT("BindingObject"));
	DetailBuilder.HideCategory(TEXT("DataInterface"));

	if (!Selection || Selection->Kind == EComputeGraphItemKind::None || Selection->Index == INDEX_NONE)
	{
		// Nothing selected — show a placeholder message.
		DetailBuilder.EditCategory(TEXT("Selection"), LOCTEXT("SelectionCategory", " "))
			.AddCustomRow(FText::GetEmpty())
			[
				SNew(STextBlock)
				.Text(LOCTEXT("NothingSelected", "Select an item from the navigator."))
				.Font(DetailBuilder.GetDetailFont())
				.ColorAndOpacity(FSlateColor::UseSubduedForeground())
			];
		return;
	}

	TSharedRef<IPropertyHandle> GraphDescHandle = DetailBuilder.GetProperty(FName("GraphDescription"));
	if (!GraphDescHandle->IsValidHandle())
	{
		return;
	}

	FName ArrayPropName;
	FText CategoryLabel;
	switch (Selection->Kind)
	{
	case EComputeGraphItemKind::Kernel:
		ArrayPropName = FName("Kernels");
		CategoryLabel = LOCTEXT("KernelCategory", "Kernel");
		break;
	case EComputeGraphItemKind::Interface:
		ArrayPropName = FName("DataInterfaces");
		CategoryLabel = LOCTEXT("InterfaceCategory", "Data Interface");
		break;
	case EComputeGraphItemKind::BindingObject:
		ArrayPropName = FName("BindingObjects");
		CategoryLabel = LOCTEXT("BindingObjectCategory", "Binding Object");
		break;
	default:
		return;
	}

	TSharedPtr<IPropertyHandle> ArrayHandle = GraphDescHandle->GetChildHandle(ArrayPropName);
	if (!ArrayHandle.IsValid() || !ArrayHandle->IsValidHandle())
	{
		return;
	}

	TSharedPtr<IPropertyHandle> ElemHandle = ArrayHandle->GetChildHandle(uint32(Selection->Index));
	if (!ElemHandle.IsValid() || !ElemHandle->IsValidHandle())
	{
		return;
	}

	IDetailCategoryBuilder& Category = DetailBuilder.EditCategory(TEXT("SelectedItem"), CategoryLabel);

	TArray<TWeakObjectPtr<UObject>> ObjectsBeingCustomized;
	DetailBuilder.GetObjectsBeingCustomized(ObjectsBeingCustomized);
	UEditableComputeGraph* Graph = (ObjectsBeingCustomized.Num() == 1) ? Cast<UEditableComputeGraph>(ObjectsBeingCustomized[0].Get()) : nullptr;

	// For the EntryPoint combo we need access to the SourceText handle.
	TSharedPtr<IPropertyHandle> SourceTextHandle;
	if (Selection->Kind == EComputeGraphItemKind::Kernel)
	{
		SourceTextHandle = ElemHandle->GetChildHandle(FName("SourceText"));
	}

	uint32 NumChildren = 0;
	ElemHandle->GetNumChildren(NumChildren);
	for (uint32 i = 0; i < NumChildren; ++i)
	{
		TSharedPtr<IPropertyHandle> ChildHandle = ElemHandle->GetChildHandle(i);
		if (!ChildHandle.IsValid() || !ChildHandle->IsValidHandle()) 
		{
			continue;
		}

		const FName PropName = ChildHandle->GetProperty() ? ChildHandle->GetProperty()->GetFName() : NAME_None;

		// Skip internal / redundant fields.
		if (PropName == FName("bOrphaned"))
		{
			continue;
		}

		// SourceText is always visible in the HLSL editor panel.
		if (Selection->Kind == EComputeGraphItemKind::Kernel && PropName == FName("SourceText")) 
		{
			continue;
		}

		// BindingObjectName combo box populated from BindingObjects.
		if (Selection->Kind == EComputeGraphItemKind::Interface && PropName == FName("BindingObjectName"))
		{
			TSharedPtr<IPropertyHandle> BindingHandle = ChildHandle;
			TSharedRef<TArray<TSharedPtr<FName>>> Options = MakeShared<TArray<TSharedPtr<FName>>>();

			Category.AddCustomRow(LOCTEXT("BindingObjectName", "Binding Object"))
				.NameContent()
				[
					BindingHandle->CreatePropertyNameWidget()
				]
				.ValueContent()
				.MinDesiredWidth(200.f)
				[
					SNew(SComboBox<TSharedPtr<FName>>)
					.OptionsSource(&Options.Get())
					.OnComboBoxOpening_Lambda([Options, Graph]()
					{
						Options->Reset();
						if (IsValid(Graph))
						{
							for (FComputeGraphDataBindingObjectDesc const& Desc : Graph->GetGraphDescription().BindingObjects)
							{
								if (!Desc.Name.IsNone())
								{
									Options->Add(MakeShared<FName>(Desc.Name));
								}
							}
						}
					})
					.OnSelectionChanged_Lambda([BindingHandle](TSharedPtr<FName> NewValue, ESelectInfo::Type)
					{
						if (BindingHandle.IsValid() && NewValue.IsValid())
						{
							BindingHandle->SetValue(*NewValue);
						}
					})
					.OnGenerateWidget_Lambda([](TSharedPtr<FName> Item) -> TSharedRef<SWidget>
					{
						return SNew(STextBlock)
							.Text(FText::FromName(Item.IsValid() ? *Item : NAME_None))
							.Font(FAppStyle::GetFontStyle("PropertyWindow.NormalFont"));
					})
					[
						SNew(STextBlock)
						.Text_Lambda([BindingHandle]() -> FText
						{
							FName Value;
							if (BindingHandle.IsValid()
								&& BindingHandle->GetValue(Value) == FPropertyAccess::Success
								&& !Value.IsNone())
							{
								return FText::FromName(Value);
							}
							return LOCTEXT("SelectBindingObject", "(select)");
						})
						.Font(FAppStyle::GetFontStyle("PropertyWindow.NormalFont"))
					]
				];
			continue;
		}

		// For the DataInterface Settings flatten the associated UComputeDataInterface subclass properties directly.
		if (Selection->Kind == EComputeGraphItemKind::Interface && PropName == FName("Settings"))
		{
			UObject* SettingsObj = nullptr;
			if (ChildHandle->GetValue(SettingsObj) == FPropertyAccess::Success && IsValid(SettingsObj))
			{
				TArray<UObject*> SettingsObjects = { SettingsObj };
				for (TFieldIterator<FProperty> It(SettingsObj->GetClass()); It; ++It)
				{
					if (It->HasAllPropertyFlags(CPF_Edit))
					{
						Category.AddExternalObjectProperty(SettingsObjects, It->GetFName());
					}
				}
			}
			continue;
		}

		// The EntryPoint combo box is populated by parsing function definitions from the HLSL source.
		if (Selection->Kind == EComputeGraphItemKind::Kernel && PropName == FName("EntryPoint"))
		{
			TSharedPtr<IPropertyHandle> EntryPointHandle = ChildHandle;
			TSharedRef<TArray<TSharedPtr<FString>>> Options = MakeShared<TArray<TSharedPtr<FString>>>();

			Category.AddCustomRow(LOCTEXT("EntryPoint", "Entry Point"))
				.NameContent()
				[
					EntryPointHandle->CreatePropertyNameWidget()
				]
				.ValueContent()
				.MinDesiredWidth(200.f)
				[
					SNew(SComboBox<TSharedPtr<FString>>)
					.OptionsSource(&Options.Get())
					.OnComboBoxOpening_Lambda([Options, SourceTextHandle]()
					{
						Options->Reset();
						FString SourceText;
						if (SourceTextHandle.IsValid())
						{
							SourceTextHandle->GetValue(SourceText);
						}
						for (FString const& FnName : FHlslExternalPinParser::FindFunctionDefinitions(SourceText))
						{
							Options->Add(MakeShared<FString>(FnName));
						}
					})
					.OnSelectionChanged_Lambda([EntryPointHandle](TSharedPtr<FString> NewValue, ESelectInfo::Type)
					{
						if (EntryPointHandle.IsValid() && NewValue.IsValid())
						{
							EntryPointHandle->SetValue(*NewValue);
						}
					})
					.OnGenerateWidget_Lambda([](TSharedPtr<FString> Item) -> TSharedRef<SWidget>
					{
						return SNew(STextBlock)
							.Text(FText::FromString(Item.IsValid() ? *Item : FString()))
							.Font(FAppStyle::GetFontStyle("PropertyWindow.NormalFont"));
					})
					[
						SNew(STextBlock)
						.Text_Lambda([EntryPointHandle]() -> FText
						{
							FString Value;
							if (EntryPointHandle.IsValid()
								&& EntryPointHandle->GetValue(Value) == FPropertyAccess::Success
								&& !Value.IsEmpty())
							{
								return FText::FromString(Value);
							}
							return LOCTEXT("SelectEntryPoint", "(select)");
						})
						.Font(FAppStyle::GetFontStyle("PropertyWindow.NormalFont"))
					]
				];
			continue;
		}

		Category.AddProperty(ChildHandle.ToSharedRef());
	}
}

#undef LOCTEXT_NAMESPACE
