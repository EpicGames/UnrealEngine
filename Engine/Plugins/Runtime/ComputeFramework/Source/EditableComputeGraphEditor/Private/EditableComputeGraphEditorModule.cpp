// Copyright Epic Games, Inc. All Rights Reserved.

#include "ComputeFramework/EditableComputeGraph.h"
#include "ComputeFramework/KernelPinCustomization.h"
#include "Modules/ModuleManager.h"
#include "PropertyEditorModule.h"

class FEditableComputeGraphEditorModule : public IModuleInterface
{
public:
	void StartupModule() override
	{
		FPropertyEditorModule& PropertyModule = FModuleManager::LoadModuleChecked<FPropertyEditorModule>("PropertyEditor");
		PropertyModule.RegisterCustomPropertyTypeLayout("KernelPin", FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FKernelPinCustomization::MakeInstance));
	}

	void ShutdownModule() override
	{
		if (FModuleManager::Get().IsModuleLoaded("PropertyEditor"))
		{
			FPropertyEditorModule& PropertyModule = FModuleManager::GetModuleChecked<FPropertyEditorModule>("PropertyEditor");
			PropertyModule.UnregisterCustomPropertyTypeLayout("KernelPin");
		}
	}
};

IMPLEMENT_MODULE(FEditableComputeGraphEditorModule, EditableComputeGraphEditor)
