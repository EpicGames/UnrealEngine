// Copyright Epic Games, Inc. All Rights Reserved.

#include "ComputeFramework/EditableComputeGraphEditorToolkit.h"

#include "ComputeFramework/ComputeKernelCompileResult.h"
#include "ComputeFramework/EditableComputeGraph.h"
#include "ComputeFramework/ComputeGraphHlslEditor.h"
#include "ComputeFramework/ComputeGraphNavigator.h"
#include "ComputeFramework/ComputeGraphView.h"
#include "ComputeFramework/HlslExternalPinParser.h"
#include "Framework/Docking/TabManager.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "IDetailsView.h"
#include "Modules/ModuleManager.h"
#include "PropertyEditorModule.h"
#include "ScopedTransaction.h"
#include "Styling/AppStyle.h"
#include "Widgets/Docking/SDockTab.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Layout/SScrollBox.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Views/SListView.h"
#include "Widgets/Views/STableRow.h"

#define LOCTEXT_NAMESPACE "ComputeFrameworkEditor"

const FName FEditableComputeGraphEditorToolkit::TabId_Navigator = TEXT("TID_Navigator");
const FName FEditableComputeGraphEditorToolkit::TabId_HlslEditor = TEXT("TID_HlslEditor");
const FName FEditableComputeGraphEditorToolkit::TabId_GraphView = TEXT("TID_GraphView");
const FName FEditableComputeGraphEditorToolkit::TabId_Details = TEXT("TID_Details");
const FName FEditableComputeGraphEditorToolkit::TabId_Output = TEXT("TID_Output");

static void RenameInterfaceInPins(FComputeGraphDesc& Desc, FName OldName, FName NewName)
{
	for (FComputeGraphKernelDesc& KernelDesc : Desc.Kernels)
	{
		for (FKernelPin& Pin : KernelDesc.Inputs)
		{
			if (Pin.DataInterfaceName == OldName)
			{
				Pin.DataInterfaceName = NewName;
			}
		}
		for (FKernelPin& Pin : KernelDesc.Outputs)
		{
			if (Pin.DataInterfaceName == OldName)
			{
				Pin.DataInterfaceName = NewName;
			}
		}
	}
}

static void RenameBindingObjectInInterfaces(FComputeGraphDesc& Desc, FName OldName, FName NewName)
{
	for (FComputeGraphDataInterfaceDesc& IfaceDesc : Desc.DataInterfaces)
	{
		if (IfaceDesc.BindingObjectName == OldName)
		{
			IfaceDesc.BindingObjectName = NewName;
		}
	}
}

TSharedRef<FEditableComputeGraphEditorToolkit> FEditableComputeGraphEditorToolkit::Create(UEditableComputeGraph* InAsset, EToolkitMode::Type InMode, TSharedPtr<IToolkitHost> InToolkitHost)
{
	TSharedRef<FEditableComputeGraphEditorToolkit> Toolkit = MakeShared<FEditableComputeGraphEditorToolkit>();
	Toolkit->Init(InAsset, InMode, InToolkitHost);
	return Toolkit;
}

void FEditableComputeGraphEditorToolkit::Init(UEditableComputeGraph* InAsset, EToolkitMode::Type InMode, TSharedPtr<IToolkitHost> InToolkitHost)
{
	Asset = InAsset;
	EditorSelection = MakeShared<FComputeGraphEditorSelection>();

	// Details view.
	FPropertyEditorModule& PropertyModule = FModuleManager::LoadModuleChecked<FPropertyEditorModule>("PropertyEditor");
	FDetailsViewArgs DetailsArgs;
	DetailsArgs.bHideSelectionTip = true;
	DetailsView = PropertyModule.CreateDetailView(DetailsArgs);
	DetailsView->SetObject(InAsset);

	// Refresh the navigator whenever any property changes in the details panel.
	// Also apply cross-reference fixups when an Interface or BindingObject Name is edited.
	DetailsView->OnFinishedChangingProperties().AddLambda([this](FPropertyChangedEvent const&)
	{
		if (Asset && EditorSelection && !CachedSelectedItemName.IsNone())
		{
			FComputeGraphDesc& Desc = Asset->GetGraphDescription();

			if (EditorSelection->Kind == EComputeGraphItemKind::Interface && Desc.DataInterfaces.IsValidIndex(EditorSelection->Index))
			{
				const FName NewName = Desc.DataInterfaces[EditorSelection->Index].Name;
				if (NewName != CachedSelectedItemName)
				{
					RenameInterfaceInPins(Desc, CachedSelectedItemName, NewName);
					CachedSelectedItemName = NewName;
					bDirty = true;
				}
			}
			else if (EditorSelection->Kind == EComputeGraphItemKind::BindingObject && Desc.BindingObjects.IsValidIndex(EditorSelection->Index))
			{
				const FName NewName = Desc.BindingObjects[EditorSelection->Index].Name;
				if (NewName != CachedSelectedItemName)
				{
					RenameBindingObjectInInterfaces(Desc, CachedSelectedItemName, NewName);
					CachedSelectedItemName = NewName;
					bDirty = true;
				}
			}
		}

		if (NavigatorWidget.IsValid())
		{
			NavigatorWidget->Refresh();
		}
		if (GraphViewWidget.IsValid())
		{
			GraphViewWidget->Refresh();
		}
	});

	// Subscribe to compile output from the asset.
	CompileOutputHandle = InAsset->OnCompileOutputChanged.AddRaw(this, &FEditableComputeGraphEditorToolkit::OnCompileOutputChanged);

	// Register per-instance customization so the details panel shows only the selected navigator item rather than the full asset.
	DetailsView->RegisterInstancedCustomPropertyLayout(
		UEditableComputeGraph::StaticClass(),
		FOnGetDetailCustomizationInstance::CreateStatic(&FEditableComputeGraphDetailCustomization::MakeInstance, EditorSelection));

	// Tab layout.
	TSharedRef<FTabManager::FLayout> Layout = FTabManager::NewLayout("EditableComputeGraphEditor_v4")
		->AddArea
		(
			FTabManager::NewPrimaryArea()
			->SetOrientation(Orient_Horizontal)
			->Split
			(
				FTabManager::NewStack()
				->SetSizeCoefficient(0.22f)
				->AddTab(TabId_Navigator, ETabState::OpenedTab)
				->SetHideTabWell(true)
			)
			->Split
			(
				FTabManager::NewSplitter()
				->SetOrientation(Orient_Vertical)
				->SetSizeCoefficient(0.50f)
				->Split
				(
					FTabManager::NewStack()
					->SetSizeCoefficient(0.50f)
					->AddTab(TabId_HlslEditor, ETabState::OpenedTab)
					->SetHideTabWell(true)
				)
				->Split
				(
					FTabManager::NewStack()
					->SetSizeCoefficient(0.25f)
					->AddTab(TabId_GraphView, ETabState::OpenedTab)
					->SetHideTabWell(true)
				)
				->Split
				(
					FTabManager::NewStack()
					->SetSizeCoefficient(0.25f)
					->AddTab(TabId_Output, ETabState::OpenedTab)
					->SetHideTabWell(true)
				)
			)
			->Split
			(
				FTabManager::NewStack()
				->SetSizeCoefficient(0.28f)
				->AddTab(TabId_Details, ETabState::OpenedTab)
				->SetHideTabWell(true)
			)
		);

	constexpr bool bCreateDefaultStandaloneMenu = true;
	constexpr bool bCreateDefaultToolbar = true;
	FAssetEditorToolkit::InitAssetEditor(
		InMode,
		InToolkitHost,
		TEXT("EditableComputeGraphEditorApp"),
		Layout,
		bCreateDefaultStandaloneMenu,
		bCreateDefaultToolbar,
		InAsset);

	ExtendToolbar();
	RegenerateMenusAndToolbars();
}

FName FEditableComputeGraphEditorToolkit::GetToolkitFName() const
{
	return TEXT("EditableComputeGraphEditor");
}

FText FEditableComputeGraphEditorToolkit::GetBaseToolkitName() const
{
	return LOCTEXT("ToolkitName", "Compute Graph Editor");
}

FString FEditableComputeGraphEditorToolkit::GetWorldCentricTabPrefix() const
{
	return TEXT("ComputeGraph ");
}

FLinearColor FEditableComputeGraphEditorToolkit::GetWorldCentricTabColorScale() const
{
	return FLinearColor(0.0f, 0.5f, 0.5f, 0.5f);
}

void FEditableComputeGraphEditorToolkit::AddReferencedObjects(FReferenceCollector& Collector)
{
	Collector.AddReferencedObject(Asset);
}

FString FEditableComputeGraphEditorToolkit::GetReferencerName() const
{
	return TEXT("FEditableComputeGraphEditorToolkit");
}

void FEditableComputeGraphEditorToolkit::RegisterTabSpawners(TSharedRef<FTabManager> const& InTabManager)
{
	FAssetEditorToolkit::RegisterTabSpawners(InTabManager);

	WorkspaceMenuCategory = InTabManager->AddLocalWorkspaceMenuCategory(LOCTEXT("WorkspaceMenuCategory", "Compute Graph Editor"));

	InTabManager->RegisterTabSpawner(TabId_Navigator, FOnSpawnTab::CreateSP(this, &FEditableComputeGraphEditorToolkit::SpawnTab_Navigator))
		.SetDisplayName(LOCTEXT("NavigatorTab", "Navigator"))
		.SetGroup(WorkspaceMenuCategory.ToSharedRef());

	InTabManager->RegisterTabSpawner(TabId_HlslEditor, FOnSpawnTab::CreateSP(this, &FEditableComputeGraphEditorToolkit::SpawnTab_HlslEditor))
		.SetDisplayName(LOCTEXT("HlslEditorTab", "HLSL Editor"))
		.SetGroup(WorkspaceMenuCategory.ToSharedRef());

	InTabManager->RegisterTabSpawner(TabId_GraphView, FOnSpawnTab::CreateSP(this, &FEditableComputeGraphEditorToolkit::SpawnTab_GraphView))
		.SetDisplayName(LOCTEXT("GraphViewTab", "Graph View"))
		.SetGroup(WorkspaceMenuCategory.ToSharedRef());

	InTabManager->RegisterTabSpawner(TabId_Details, FOnSpawnTab::CreateSP(this, &FEditableComputeGraphEditorToolkit::SpawnTab_Details))
		.SetDisplayName(LOCTEXT("DetailsTab", "Details"))
		.SetGroup(WorkspaceMenuCategory.ToSharedRef());

	InTabManager->RegisterTabSpawner(TabId_Output, FOnSpawnTab::CreateSP(this, &FEditableComputeGraphEditorToolkit::SpawnTab_Output))
		.SetDisplayName(LOCTEXT("OutputTab", "Output"))
		.SetGroup(WorkspaceMenuCategory.ToSharedRef());
}

void FEditableComputeGraphEditorToolkit::UnregisterTabSpawners(TSharedRef<FTabManager> const& InTabManager)
{
	FAssetEditorToolkit::UnregisterTabSpawners(InTabManager);
	InTabManager->UnregisterTabSpawner(TabId_Navigator);
	InTabManager->UnregisterTabSpawner(TabId_HlslEditor);
	InTabManager->UnregisterTabSpawner(TabId_GraphView);
	InTabManager->UnregisterTabSpawner(TabId_Details);
	InTabManager->UnregisterTabSpawner(TabId_Output);

	if (Asset != nullptr)
	{
		Asset->OnCompileOutputChanged.Remove(CompileOutputHandle);
	}
}

TSharedRef<SDockTab> FEditableComputeGraphEditorToolkit::SpawnTab_Navigator(FSpawnTabArgs const& Args)
{
	NavigatorWidget = SNew(SComputeGraphNavigator)
		.Asset(Asset)
		.OnItemSelected(FOnComputeGraphItemSelected::CreateSP(this, &FEditableComputeGraphEditorToolkit::OnItemSelected))
		.OnAddItem(FOnComputeGraphAddItem::CreateSP(this, &FEditableComputeGraphEditorToolkit::OnAddItem))
		.OnDeleteItem(FOnComputeGraphDeleteItem::CreateSP(this, &FEditableComputeGraphEditorToolkit::OnDeleteItem))
		.OnDuplicateItem(FOnComputeGraphDuplicateItem::CreateSP(this, &FEditableComputeGraphEditorToolkit::OnDuplicateItem))
		.OnRenameItem(FOnComputeGraphRenameItem::CreateSP(this, &FEditableComputeGraphEditorToolkit::OnRenameItem))
		.OnKernelSelected(FOnComputeGraphKernelSelected::CreateSP(this, &FEditableComputeGraphEditorToolkit::OnKernelSelected));

	return SNew(SDockTab)
		.TabRole(ETabRole::PanelTab)
		[
			NavigatorWidget.ToSharedRef()
		];
}

TSharedRef<SDockTab> FEditableComputeGraphEditorToolkit::SpawnTab_HlslEditor(FSpawnTabArgs const& Args)
{
	HlslEditorWidget = SNew(SComputeGraphHlslEditor)
		.Asset(Asset)
		.OnTextCommitted_WithKernel(FOnComputeGraphHlslTextCommitted::CreateSP(this, &FEditableComputeGraphEditorToolkit::OnHlslTextCommitted))
		.OnTextChanged(FOnComputeGraphHlslTextChanged::CreateLambda([this]() { bDirty = true; }))
		.OnCompileRequested(FOnComputeGraphHlslCompileRequested::CreateSP(this, &FEditableComputeGraphEditorToolkit::CompileGraph));

	return SNew(SDockTab)
		.TabRole(ETabRole::PanelTab)
		[
			HlslEditorWidget.ToSharedRef()
		];
}

TSharedRef<SDockTab> FEditableComputeGraphEditorToolkit::SpawnTab_GraphView(FSpawnTabArgs const& Args)
{
	GraphViewWidget = SNew(SComputeGraphView)
		.Asset(Asset)
		.OnNodeClicked(this, &FEditableComputeGraphEditorToolkit::OnGraphNodeClicked);

	return SNew(SDockTab)
		.TabRole(ETabRole::PanelTab)
		[
			SNew(SScrollBox)
			.Orientation(Orient_Horizontal)
			+ SScrollBox::Slot()
			[
				SNew(SScrollBox)
				.Orientation(Orient_Vertical)
				+ SScrollBox::Slot()
				[
					GraphViewWidget.ToSharedRef()
				]
			]
		];
}

TSharedRef<SDockTab> FEditableComputeGraphEditorToolkit::SpawnTab_Details(FSpawnTabArgs const& Args)
{
	return SNew(SDockTab)
		.TabRole(ETabRole::PanelTab)
		[
			DetailsView.ToSharedRef()
		];
}

TSharedRef<SDockTab> FEditableComputeGraphEditorToolkit::SpawnTab_Output(FSpawnTabArgs const& Args)
{
	SAssignNew(OutputListView, SListView<TSharedPtr<FComputeKernelCompileMessage>>)
		.ListItemsSource(&OutputMessages)
		.OnGenerateRow(this, &FEditableComputeGraphEditorToolkit::GenerateOutputRow)
		.OnSelectionChanged(this, &FEditableComputeGraphEditorToolkit::OnOutputSelectionChanged)
		.SelectionMode(ESelectionMode::Single);

	return SNew(SDockTab)
		.TabRole(ETabRole::PanelTab)
		[
			SNew(SBorder)
			.BorderImage(FAppStyle::Get().GetBrush("ToolPanel.GroupBorder"))
			.Padding(4.f)
			[
				OutputListView.ToSharedRef()
			]
		];
}

TSharedRef<ITableRow> FEditableComputeGraphEditorToolkit::GenerateOutputRow(
	TSharedPtr<FComputeKernelCompileMessage> Message,
	TSharedRef<STableViewBase> const& OwnerTable)
{
	FSlateColor Color = FSlateColor::UseForeground();
	FString Text;

	if (Message.IsValid())
	{
		switch (Message->Type)
		{
		case FComputeKernelCompileMessage::EMessageType::Error:
			Color = FSlateColor(FLinearColor(1.f, 0.3f, 0.3f));
			break;
		case FComputeKernelCompileMessage::EMessageType::Warning:
			Color = FSlateColor(FLinearColor(1.f, 0.85f, 0.2f));
			break;
		default:
			break;
		}

		Text = Message->Line >= 0
			? FString::Printf(TEXT("(%d): %s"), Message->Line, *Message->Text)
			: Message->Text;
	}

	return SNew(STableRow<TSharedPtr<FComputeKernelCompileMessage>>, OwnerTable)
		.Padding(FMargin(4.f, 1.f))
		[
			SNew(STextBlock)
			.Text(FText::FromString(Text))
			.Font(FAppStyle::GetFontStyle("PropertyWindow.NormalFont"))
			.ColorAndOpacity(Color)
		];
}

void FEditableComputeGraphEditorToolkit::OnCompileOutputChanged(TArray<FComputeKernelCompileMessage> const& NewMessages)
{
	if (NewMessages.IsEmpty())
	{
		OutputMessages.Reset();
	}
	else
	{
		for (FComputeKernelCompileMessage const& Msg : NewMessages)
		{
			OutputMessages.Add(MakeShared<FComputeKernelCompileMessage>(Msg));
		}
	}
	if (OutputListView.IsValid())
	{
		OutputListView->RequestListRefresh();
	}
}

void FEditableComputeGraphEditorToolkit::OnOutputSelectionChanged(TSharedPtr<FComputeKernelCompileMessage> Selected, ESelectInfo::Type /*SelectInfo*/)
{
	if (!Selected.IsValid() || !Asset || !NavigatorWidget.IsValid())
	{
		return;
	}

	// Parse the [KernelName] prefix added by OnKernelCompilationComplete.
	if (!Selected->Text.StartsWith(TEXT("[")))
	{
		return;
	}
	int32 CloseBracket = INDEX_NONE;
	Selected->Text.FindChar(TEXT(']'), CloseBracket);
	if (CloseBracket <= 1)
	{
		return;
	}
	const FString ParsedName = Selected->Text.Mid(1, CloseBracket - 1);

	TArray<FComputeGraphKernelDesc> const& Kernels = Asset->GetGraphDescription().Kernels;
	const int32 KernelIdx = Kernels.IndexOfByPredicate([&](FComputeGraphKernelDesc const& K) { return K.Name.ToString() == ParsedName; });
	if (KernelIdx != INDEX_NONE)
	{
		NavigatorWidget->SetSelectedItem(EComputeGraphItemKind::Kernel, KernelIdx);
	}
}

void FEditableComputeGraphEditorToolkit::ExtendToolbar()
{
	TSharedPtr<FExtender> Extender = MakeShared<FExtender>();
	Extender->AddToolBarExtension(
		"Asset",
		EExtensionHook::After,
		GetToolkitCommands(),
		FToolBarExtensionDelegate::CreateLambda([this](FToolBarBuilder& ToolbarBuilder)
		{
			ToolbarBuilder.AddToolBarButton(
				FUIAction(
					FExecuteAction::CreateSP(this, &FEditableComputeGraphEditorToolkit::CompileGraph),
					FCanExecuteAction::CreateLambda([this]() { return GetCompileState() != ECompileState::UpToDate; })),
				NAME_None,
				LOCTEXT("CompileButton", "Compile"),
				LOCTEXT("CompileButtonTooltip", "Rebuild the compute graph and trigger shader compilation."),
				TAttribute<FSlateIcon>(this, &FEditableComputeGraphEditorToolkit::GetCompileStatusIcon));
		}));
	AddToolbarExtender(Extender);
}

FEditableComputeGraphEditorToolkit::ECompileState FEditableComputeGraphEditorToolkit::GetCompileState() const
{
	if (!bDirty)
	{
		return ECompileState::UpToDate;
	}

	// Validate: Every DataInterface must have a BindingObject set, and every kernel pin must have all three name fields filled.
	if (Asset)
	{
		FComputeGraphDesc const& Desc = Asset->GetGraphDescription();

		for (FComputeGraphDataInterfaceDesc const& Iface : Desc.DataInterfaces)
		{
			if (Iface.BindingObjectName.IsNone())
			{
				return ECompileState::Broken;
			}
		}

		for (FComputeGraphKernelDesc const& Kernel : Desc.Kernels)
		{
			auto PinIncomplete = [](FKernelPin const& Pin) -> bool
			{
				return Pin.KernelFunctionName.IsEmpty()	|| Pin.DataInterfaceName.IsNone() || Pin.DataInterfaceFunctionName.IsEmpty();
			};

			for (FKernelPin const& Pin : Kernel.Inputs)
			{
				if (PinIncomplete(Pin))
				{
					return ECompileState::Broken;
				}
			}
			for (FKernelPin const& Pin : Kernel.Outputs)
			{
				if (PinIncomplete(Pin))
				{
					return ECompileState::Broken;
				}
			}
		}
	}

	return ECompileState::NeedsCompile;
}

FSlateIcon FEditableComputeGraphEditorToolkit::GetCompileStatusIcon() const
{
	static const FName Background(TEXT("AssetEditor.CompileStatus.Background"));
	static const FName OverlayGood(TEXT("AssetEditor.CompileStatus.Overlay.Good"));
	static const FName OverlayUnknown(TEXT("AssetEditor.CompileStatus.Overlay.Unknown"));
	static const FName OverlayError(TEXT("AssetEditor.CompileStatus.Overlay.Error"));

	const FName StyleSet = FAppStyle::GetAppStyleSetName();

	switch (GetCompileState())
	{
	case ECompileState::UpToDate:
		return FSlateIcon(StyleSet, Background, OverlayGood);
	case ECompileState::Broken:
		return FSlateIcon(StyleSet, Background, OverlayError);
	default:
		return FSlateIcon(StyleSet, Background, OverlayUnknown);
	}
}

void FEditableComputeGraphEditorToolkit::OnItemSelected(EComputeGraphItemKind Kind, int32 Index, FName Name)
{
	if (EditorSelection)
	{
		EditorSelection->Kind = Kind;
		EditorSelection->Index = Index;
	}

	// Snapshot the name so OnFinishedChangingProperties can detect in-panel renames.
	CachedSelectedItemName = NAME_None;
	if (Asset && (Kind == EComputeGraphItemKind::Interface || Kind == EComputeGraphItemKind::BindingObject))
	{
		CachedSelectedItemName = Name;
	}

	DetailsView->ForceRefresh();

	// Also drive the HLSL editor / graph view depending on selection kind.
	if (Kind == EComputeGraphItemKind::Kernel)
	{
		OnKernelSelected(Name);
		if (GraphViewWidget.IsValid()) 
		{ 
			GraphViewWidget->SetHighlightedInterface(NAME_None); 
		}
	}
	else if (Kind == EComputeGraphItemKind::Interface)
	{
		if (GraphViewWidget.IsValid())
		{
			GraphViewWidget->SetHighlightedKernel(NAME_None);
			GraphViewWidget->SetHighlightedInterface(Name);
		}
	}
	else
	{
		if (GraphViewWidget.IsValid())
		{
			GraphViewWidget->SetHighlightedKernel(NAME_None);
			GraphViewWidget->SetHighlightedInterface(NAME_None);
		}
	}
}

void FEditableComputeGraphEditorToolkit::OnGraphNodeClicked(FName NodeName, bool bIsKernel)
{
	if (Asset == nullptr)
	{
		return;
	}

	FComputeGraphDesc const& Desc = Asset->GetGraphDescription();

	if (bIsKernel)
	{
		for (int32 KernelIndex = 0; KernelIndex < Desc.Kernels.Num(); ++KernelIndex)
		{
			if (Desc.Kernels[KernelIndex].Name == NodeName)
			{
				NavigatorWidget->SetSelectedItem(EComputeGraphItemKind::Kernel, KernelIndex);
				OnItemSelected(EComputeGraphItemKind::Kernel, KernelIndex, NodeName);
				return;
			}
		}
	}
	else
	{
		for (int32 DataInterfaceIndex = 0; DataInterfaceIndex < Desc.DataInterfaces.Num(); ++DataInterfaceIndex)
		{
			if (Desc.DataInterfaces[DataInterfaceIndex].Name == NodeName)
			{
				NavigatorWidget->SetSelectedItem(EComputeGraphItemKind::Interface, DataInterfaceIndex);
				OnItemSelected(EComputeGraphItemKind::Interface, DataInterfaceIndex, NodeName);
				return;
			}
		}
	}
}

void FEditableComputeGraphEditorToolkit::RefreshHlslBoundFunctions(FName KernelName)
{
	if (Asset == nullptr || !HlslEditorWidget.IsValid()) 
	{
		return;
	}

	TArray<FString> BoundNames;
	for (FComputeGraphKernelDesc const& KernelDesc : Asset->GetGraphDescription().Kernels)
	{
		if (KernelDesc.Name != KernelName)
		{
			continue;
		}
		if (!KernelDesc.EntryPoint.IsEmpty())
		{
			BoundNames.Add(KernelDesc.EntryPoint);
		}
		for (FKernelPin const& Pin : KernelDesc.Inputs)
		{
			if (!Pin.KernelFunctionName.IsEmpty())
			{
				BoundNames.Add(Pin.KernelFunctionName);
			}
		}
		for (FKernelPin const& Pin : KernelDesc.Outputs)
		{
			if (!Pin.KernelFunctionName.IsEmpty())
			{
				BoundNames.Add(Pin.KernelFunctionName);
			}
		}
		break;
	}
	HlslEditorWidget->SetBoundFunctions(BoundNames);
}

void FEditableComputeGraphEditorToolkit::OnKernelSelected(FName KernelName)
{
	if (HlslEditorWidget.IsValid())
	{
		HlslEditorWidget->SetActiveKernel(KernelName);
		RefreshHlslBoundFunctions(KernelName);
	}
	if (GraphViewWidget.IsValid())
	{
		GraphViewWidget->SetHighlightedKernel(KernelName);
	}
}

FName FEditableComputeGraphEditorToolkit::GenerateUniqueName(TArray<FName> const& Existing, FStringView Base)
{
	FString Candidate(Base);
	int32 Suffix = 0;
	while (Existing.Contains(FName(*Candidate)))
	{
		Candidate = FString::Printf(TEXT("%s%d"), *FString(Base), ++Suffix);
	}
	return FName(*Candidate);
}

void FEditableComputeGraphEditorToolkit::OnAddItem(EComputeGraphItemKind Kind)
{
	if (Asset == nullptr) 
	{
		return;
	}

	FScopedTransaction Transaction(LOCTEXT("AddItem", "Add Compute Graph Item"));
	Asset->Modify();

	FComputeGraphDesc& Desc = Asset->GetGraphDescription();
	int32 NewIndex = INDEX_NONE;

	switch (Kind)
	{
	case EComputeGraphItemKind::Kernel:
	{
		TArray<FName> Existing;
		for (FComputeGraphKernelDesc const& K : Desc.Kernels)
		{
			Existing.Add(K.Name);
		}

		// Add stub kernel main function.
		FComputeGraphKernelDesc& New = Desc.Kernels.AddDefaulted_GetRef();
		New.Name = GenerateUniqueName(Existing, TEXT("NewKernel"));
		New.EntryPoint = TEXT("Main");
		New.GroupSize = FIntVector(64, 1, 1);
		New.SourceText = TEXT("void Main(uint3 DTId : SV_DispatchThreadID, uint3 GTId : SV_GroupThreadID, uint3 GId : SV_GroupID)\n{\n}\n");
		NewIndex = Desc.Kernels.Num() - 1;
		break;
	}
	case EComputeGraphItemKind::Interface:
	{
		TArray<FName> Existing;
		for (FComputeGraphDataInterfaceDesc const& I : Desc.DataInterfaces)
		{
			Existing.Add(I.Name);
		}

		FComputeGraphDataInterfaceDesc& New = Desc.DataInterfaces.AddDefaulted_GetRef();
		New.Name = GenerateUniqueName(Existing, TEXT("NewInterface"));
		NewIndex = Desc.DataInterfaces.Num() - 1;
		break;
	}
	case EComputeGraphItemKind::BindingObject:
	{
		TArray<FName> Existing;
		for (FComputeGraphDataBindingObjectDesc const& B : Desc.BindingObjects)
		{
			Existing.Add(B.Name);
		}

		FComputeGraphDataBindingObjectDesc& New = Desc.BindingObjects.AddDefaulted_GetRef();
		New.Name = GenerateUniqueName(Existing, TEXT("NewBindingObject"));
		NewIndex = Desc.BindingObjects.Num() - 1;
		break;
	}
	default:
		return;
	}

	RefreshNavigatorAndDetails();
	if (NavigatorWidget.IsValid())
	{
		NavigatorWidget->SetSelectedItem(Kind, NewIndex);
	}
}

void FEditableComputeGraphEditorToolkit::OnDeleteItem(EComputeGraphItemKind Kind, int32 Index)
{
	if (Asset == nullptr)
	{
		return;
	}

	FScopedTransaction Transaction(LOCTEXT("Toolkit_DeleteItem", "Delete Compute Graph Item"));
	Asset->Modify();

	FComputeGraphDesc& Desc = Asset->GetGraphDescription();
	switch (Kind)
	{
	case EComputeGraphItemKind::Kernel:
		if (Desc.Kernels.IsValidIndex(Index)) 
		{
			Desc.Kernels.RemoveAt(Index);
		}
		break;
	case EComputeGraphItemKind::Interface:
		if (Desc.DataInterfaces.IsValidIndex(Index)) 
		{
			Desc.DataInterfaces.RemoveAt(Index);
		}
		break;
	case EComputeGraphItemKind::BindingObject:
		if (Desc.BindingObjects.IsValidIndex(Index)) 
		{
			Desc.BindingObjects.RemoveAt(Index);
		}
		break;
	default:
		break;
	}

	// Clear selection if we deleted the selected item.
	if (EditorSelection && EditorSelection->Kind == Kind && EditorSelection->Index == Index)
	{
		EditorSelection->Kind = EComputeGraphItemKind::None;
		EditorSelection->Index = INDEX_NONE;
	}

	RefreshNavigatorAndDetails();
}

void FEditableComputeGraphEditorToolkit::OnDuplicateItem(EComputeGraphItemKind Kind, int32 Index)
{
	if (Asset == nullptr) 
	{
		return;
	}

	FScopedTransaction Transaction(LOCTEXT("Toolkit_DuplicateItem", "Duplicate Compute Graph Item"));
	Asset->Modify();

	FComputeGraphDesc& Desc = Asset->GetGraphDescription();
	int32 NewIndex = INDEX_NONE;

	switch (Kind)
	{
	case EComputeGraphItemKind::Kernel:
		if (Desc.Kernels.IsValidIndex(Index))
		{
			TArray<FName> Existing;
			for (FComputeGraphKernelDesc const& Kernel : Desc.Kernels)
			{
				Existing.Add(Kernel.Name);
			}

			FComputeGraphKernelDesc Copy = Desc.Kernels[Index];
			Copy.Name = GenerateUniqueName(Existing, Copy.Name.ToString() + TEXT("_Copy"));
			Desc.Kernels.Add(MoveTemp(Copy));
			NewIndex = Desc.Kernels.Num() - 1;
		}
		break;
	case EComputeGraphItemKind::Interface:
		if (Desc.DataInterfaces.IsValidIndex(Index))
		{
			TArray<FName> Existing;
			for (FComputeGraphDataInterfaceDesc const& DataInterface : Desc.DataInterfaces)
			{
				Existing.Add(DataInterface.Name);
			}

			FComputeGraphDataInterfaceDesc Copy = Desc.DataInterfaces[Index];
			Copy.Name = GenerateUniqueName(Existing, Copy.Name.ToString() + TEXT("_Copy"));
			Desc.DataInterfaces.Add(MoveTemp(Copy));
			NewIndex = Desc.DataInterfaces.Num() - 1;
		}
		break;
	case EComputeGraphItemKind::BindingObject:
		if (Desc.BindingObjects.IsValidIndex(Index))
		{
			TArray<FName> Existing;
			for (FComputeGraphDataBindingObjectDesc const& BindingObject : Desc.BindingObjects)
			{
				Existing.Add(BindingObject.Name);
			}

			FComputeGraphDataBindingObjectDesc Copy = Desc.BindingObjects[Index];
			Copy.Name = GenerateUniqueName(Existing, Copy.Name.ToString() + TEXT("_Copy"));
			Desc.BindingObjects.Add(MoveTemp(Copy));
			NewIndex = Desc.BindingObjects.Num() - 1;
		}
		break;
	default:
		break;
	}

	RefreshNavigatorAndDetails();
	if (NavigatorWidget.IsValid() && NewIndex != INDEX_NONE)
	{
		NavigatorWidget->SetSelectedItem(Kind, NewIndex);
	}
}

void FEditableComputeGraphEditorToolkit::OnRenameItem(EComputeGraphItemKind Kind, int32 Index, FName NewName)
{
	if (Asset == nullptr)
	{
		return;
	}

	FScopedTransaction Transaction(LOCTEXT("Toolkit_RenameItem", "Rename Compute Graph Item"));
	Asset->Modify();

	FComputeGraphDesc& Desc = Asset->GetGraphDescription();
	switch (Kind)
	{
	case EComputeGraphItemKind::Kernel:
		if (Desc.Kernels.IsValidIndex(Index))
		{
			Desc.Kernels[Index].Name = NewName;
		}
		break;
	case EComputeGraphItemKind::Interface:
		if (Desc.DataInterfaces.IsValidIndex(Index))
		{
			const FName OldName = Desc.DataInterfaces[Index].Name;
			Desc.DataInterfaces[Index].Name = NewName;
			RenameInterfaceInPins(Desc, OldName, NewName);
			if (CachedSelectedItemName == OldName)
			{
				CachedSelectedItemName = NewName;
			}
		}
		break;
	case EComputeGraphItemKind::BindingObject:
		if (Desc.BindingObjects.IsValidIndex(Index))
		{
			const FName OldName = Desc.BindingObjects[Index].Name;
			Desc.BindingObjects[Index].Name = NewName;

			RenameBindingObjectInInterfaces(Desc, OldName, NewName);
			if (CachedSelectedItemName == OldName)
			{
				CachedSelectedItemName = NewName;
			}
		}
		break;
	default:
		break;
	}

	RefreshNavigatorAndDetails();
}

void FEditableComputeGraphEditorToolkit::RefreshNavigatorAndDetails()
{
	bDirty = true;

	if (NavigatorWidget.IsValid())
	{
		NavigatorWidget->Refresh();
	}
	if (GraphViewWidget.IsValid())
	{
		GraphViewWidget->Refresh();
	}
	DetailsView->ForceRefresh();
}

void FEditableComputeGraphEditorToolkit::OnHlslTextCommitted(FName KernelName, FString const& NewText)
{
	if (Asset == nullptr) 
	{
		return;
	}

	Asset->Modify();

	FComputeGraphDesc& Desc = Asset->GetGraphDescription();
	for (FComputeGraphKernelDesc& KernelDesc : Desc.Kernels)
	{
		if (KernelDesc.Name != KernelName) 
		{
			continue;
		}

		KernelDesc.SourceText = NewText;

		TArray<FHlslExternalPinParser::FPinDeclaration> const Decls = FHlslExternalPinParser::FindExternalPins(NewText);

		TSet<FString> DeclaredInputNames, DeclaredOutputNames;
		for (FHlslExternalPinParser::FPinDeclaration const& Decl : Decls)
		{
			(Decl.bIsOutput ? DeclaredOutputNames : DeclaredInputNames).Add(Decl.FunctionName);
		}

		// Sync pins for one direction.
		// Consider 1 new + 1 unmatched as a rename and update in place.
		// Otherwise add new pins and mark unmatched ones as orphaned.
		auto SyncPins = [](TArray<FKernelPin>& Pins, TSet<FString> const& DeclaredNames)
		{
			TArray<FString> ToAdd;
			TArray<int32> UnmatchedIndices;
			TSet<FString> ExistingNames;
			
			for (FKernelPin const& Pin : Pins)
			{
				ExistingNames.Add(Pin.KernelFunctionName);
			}

			for (FString const& Name : DeclaredNames)
			{
				if (!ExistingNames.Contains(Name))
				{
					ToAdd.Add(Name);
				}
			}

			for (int32 i = 0; i < Pins.Num(); ++i)
			{
				if (!Pins[i].KernelFunctionName.IsEmpty() && !DeclaredNames.Contains(Pins[i].KernelFunctionName))
				{
					UnmatchedIndices.Add(i);
				}
			}

			if (ToAdd.Num() == 1 && UnmatchedIndices.Num() == 1)
			{
				// Unambiguous rename so update the existing pin in place and preserve bindings.
				Pins[UnmatchedIndices[0]].KernelFunctionName = ToAdd[0];
				Pins[UnmatchedIndices[0]].bOrphaned = false;
			}
			else
			{
				for (FString const& Name : ToAdd)
				{
					Pins.AddDefaulted_GetRef().KernelFunctionName = Name;
				}
				for (int32 Idx : UnmatchedIndices)
				{
					Pins[Idx].bOrphaned = true;
				}
			}
		};

		SyncPins(KernelDesc.Inputs, DeclaredInputNames);
		SyncPins(KernelDesc.Outputs, DeclaredOutputNames);

		// Sync EntryPoint.
		// If it no longer appears as a function in the HLSL but exactly one non-DI_ void function does then treat as a rename.
		if (!KernelDesc.EntryPoint.IsEmpty() && !FHlslExternalPinParser::FunctionExistsInText(NewText, KernelDesc.EntryPoint))
		{
			TArray<FString> Candidates = FHlslExternalPinParser::FindFunctionDefinitions(NewText);
			Candidates.RemoveAll([](FString const& Name) { return Name.StartsWith(TEXT("DI_")); });
			if (Candidates.Num() == 1)
			{
				KernelDesc.EntryPoint = Candidates[0];
			}
		}

		bDirty = true;
		break;
	}

	if (HlslEditorWidget.IsValid())
	{
		RefreshHlslBoundFunctions(KernelName);
	}
	if (NavigatorWidget.IsValid())
	{
		NavigatorWidget->Refresh();
	}
	if (GraphViewWidget.IsValid())
	{
		GraphViewWidget->Refresh();
	}
	DetailsView->ForceRefresh();
}

void FEditableComputeGraphEditorToolkit::CompileGraph()
{
	if (Asset == nullptr) 
	{
		return;
	}

	// Flush any uncommitted HLSL text so the compile sees the latest source.
	if (HlslEditorWidget.IsValid())
	{
		HlslEditorWidget->CommitCurrentText();
	}

	Asset->RebuildGraph();

	bDirty = false;
}

#undef LOCTEXT_NAMESPACE
