// Copyright Epic Games, Inc. All Rights Reserved.

#include "ComputeFramework/EditableComputeGraphFactory.h"

#include "ComputeFramework/EditableComputeGraph.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(EditableComputeGraphFactory)

UEditableComputeGraphFactory::UEditableComputeGraphFactory()
{
	SupportedClass = UEditableComputeGraph::StaticClass();
	bCreateNew = true;
	bEditAfterNew = true;
}

UObject* UEditableComputeGraphFactory::FactoryCreateNew(UClass* InClass, UObject* InParent, FName InName, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn)
{
	return NewObject<UEditableComputeGraph>(InParent, InClass, InName, Flags);
}
