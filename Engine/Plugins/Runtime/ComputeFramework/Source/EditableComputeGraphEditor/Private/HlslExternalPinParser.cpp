// Copyright Epic Games, Inc. All Rights Reserved.

#include "ComputeFramework/HlslExternalPinParser.h"

#include "Containers/Set.h"
#include "Containers/StringView.h"

namespace HlslExternalPinParser {

/** Replace all content inside line and block comments with spaces, preserving newlines so that line numbers are unchanged. */
static FString StripComments(FStringView Src)
{
	FString Out;
	Out.Reserve(Src.Len());

	const int32 Len = Src.Len();
	int32 Pos = 0;
	while (Pos < Len)
	{
		if (Pos + 1 < Len && Src[Pos] == TEXT('/') && Src[Pos + 1] == TEXT('/'))
		{
			// Line comment: replace up to (but not including) the newline.
			Pos += 2;
			while (Pos < Len && Src[Pos] != TEXT('\n'))
			{
				Out += TEXT(' ');
				++Pos;
			}
		}
		else if (Pos + 1 < Len && Src[Pos] == TEXT('/') && Src[Pos + 1] == TEXT('*'))
		{
			// Block comment: replace content; preserve newlines.
			Pos += 2;
			Out += TEXT(' ');
			Out += TEXT(' ');
			while (Pos + 1 < Len && !(Src[Pos] == TEXT('*') && Src[Pos + 1] == TEXT('/')))
			{
				Out += (Src[Pos] == TEXT('\n')) ? TEXT('\n') : TEXT(' ');
				++Pos;
			}
			if (Pos + 1 < Len)
			{
				Out += TEXT(' ');
				Out += TEXT(' ');
				Pos += 2;
			}
		}
		else
		{
			Out += Src[Pos++];
		}
	}
	return Out;
}

} // namespace HlslExternalPinParser

TArray<FHlslExternalPinParser::FPinDeclaration> FHlslExternalPinParser::FindExternalPins(FStringView SourceText)
{
	TArray<FHlslExternalPinParser::FPinDeclaration> Result;
	TSet<FString> Seen;

	const FString Stripped = HlslExternalPinParser::StripComments(SourceText);
	TCHAR const* Data = *Stripped;
	const int32 Len = Stripped.Len();

	// Scan for "DI_" followed by an identifier character.
	int32 Pos = 0;
	while (Pos < Len)
	{
		if (Pos + 3 < Len && Data[Pos] == TEXT('D') && Data[Pos + 1] == TEXT('I') && Data[Pos + 2] == TEXT('_') && (FChar::IsAlpha(Data[Pos + 3]) || Data[Pos + 3] == TEXT('_')))
		{
			// Ensure DI_ is not the tail of a longer identifier (e.g. FOO_DI_Bar).
			const bool bValidStart = (Pos == 0)	|| !TChar<TCHAR>::IsIdentifier(Data[Pos - 1]);

			if (bValidStart)
			{
				// Collect the full identifier starting at 'D'.
				const int32 NameStart = Pos;
				Pos += 3; // skip "DI_"
				while (Pos < Len && TChar<TCHAR>::IsIdentifier(Data[Pos]))
				{
					++Pos;
				}
				const FString FuncName(Pos - NameStart, Data + NameStart);

				// Must be followed by '(' (allowing whitespace between).
				int32 AfterNamePos = Pos;
				while (AfterNamePos < Len && (Data[AfterNamePos] == TEXT(' ') || Data[AfterNamePos] == TEXT('\t')))
				{
					++AfterNamePos;
				}

				if (AfterNamePos < Len && Data[AfterNamePos] == TEXT('(') && !Seen.Contains(FuncName))
				{
					Seen.Add(FuncName);

					// Infer direction by scanning backward to the start of the statement. A "statement start" is a newline, '{', '}', or ';'.
					// If only whitespace (spaces/tabs/carriage-returns) precede DI_ on the current line, the call is a standalone statement → void → output pin.
					// If anything else appears before it, the result is used -> input pin.
					bool bStandaloneStatement = true;
					for (int32 ScanBack = NameStart - 1; ScanBack >= 0; --ScanBack)
					{
						const TCHAR PrecedingChar = Data[ScanBack];
						if (PrecedingChar == TEXT('\n') || PrecedingChar == TEXT('{') || PrecedingChar == TEXT('}') || PrecedingChar == TEXT(';'))
						{
							break;
						}
						if (PrecedingChar != TEXT(' ') && PrecedingChar != TEXT('\t') && PrecedingChar != TEXT('\r'))
						{
							bStandaloneStatement = false;
							break;
						}
					}

					Result.Add({ FuncName, /*bIsOutput=*/bStandaloneStatement });
				}
				continue; // Pos already advanced past the identifier
			}
		}
		++Pos;
	}

	return Result;
}

TArray<FString> FHlslExternalPinParser::FindFunctionDefinitions(FStringView SourceText)
{
	TArray<FString> Result;

	const FString Stripped = HlslExternalPinParser::StripComments(SourceText);
	TCHAR const* Data = *Stripped;
	const int32 Len = Stripped.Len();

	int32 BraceDepth = 0;
	int32 Pos = 0;
	TArray<FString> Words;

	auto SkipAllWhitespace = [&]()
	{
		while (Pos < Len && (Data[Pos] == TEXT(' ') || Data[Pos] == TEXT('\t') || Data[Pos] == TEXT('\r') || Data[Pos] == TEXT('\n')))
		{
			++Pos;
		}
	};

	while (Pos < Len)
	{
		const TCHAR CurrentChar = Data[Pos];

		if (CurrentChar == TEXT('{'))
		{
			++BraceDepth;
			++Pos;
			continue;
		}
		if (CurrentChar == TEXT('}'))
		{
			if (BraceDepth > 0)
			{
				--BraceDepth;
			}
			++Pos;
			continue;
		}

		// Only scan at global scope.
		if (BraceDepth > 0)
		{
			++Pos;
			continue;
		}

		// Skip whitespace, semicolons, and attribute blocks.
		if (CurrentChar == TEXT(' ') || CurrentChar == TEXT('\t') || CurrentChar == TEXT('\r') || CurrentChar == TEXT('\n') || CurrentChar == TEXT(';'))
		{
			++Pos;
			continue;
		}
		if (CurrentChar == TEXT('['))
		{
			++Pos;
			while (Pos < Len && Data[Pos] != TEXT(']'))
			{
				++Pos;
			}
			if (Pos < Len)
			{
				++Pos;
			}
			continue;
		}

		if (FChar::IsAlpha(CurrentChar) || CurrentChar == TEXT('_'))
		{
			// Collect whitespace-separated identifier tokens up to '('.
			Words.Reset();
			while (Pos < Len)
			{
				while (Pos < Len && (Data[Pos] == TEXT(' ') || Data[Pos] == TEXT('\t')))
				{
					++Pos;
				}
				if (Pos >= Len)
				{
					break;
				}
				if (Data[Pos] == TEXT('('))
				{
					break;
				}
				if (FChar::IsAlpha(Data[Pos]) || Data[Pos] == TEXT('_'))
				{
					const int32 WordStart = Pos;
					while (Pos < Len && TChar<TCHAR>::IsIdentifier(Data[Pos]))
					{
						++Pos;
					}
					Words.Add(FString(Pos - WordStart, Data + WordStart));
				}
				else
				{
					Words.Reset();
					break;
				}
			}

			if (Words.Num() < 2 || Pos >= Len || Data[Pos] != TEXT('('))
			{
				continue;
			}

			const FString FuncName = Words.Last();

			// Skip the parameter list.
			int32 ParenDepth = 1;
			++Pos;
			while (Pos < Len && ParenDepth > 0)
			{
				if (Data[Pos] == TEXT('('))
				{
					++ParenDepth;
				}
				else if (Data[Pos] == TEXT(')'))
				{
					--ParenDepth;
				}
				++Pos;
			}

			// Skip optional semantic (': SV_Something') and whitespace.
			SkipAllWhitespace();
			if (Pos < Len && Data[Pos] == TEXT(':'))
			{
				while (Pos < Len && Data[Pos] != TEXT('{') && Data[Pos] != TEXT(';'))
				{
					++Pos;
				}
			}
			SkipAllWhitespace();

			if (Pos < Len && Data[Pos] == TEXT('{'))
			{
				Result.Add(FuncName);
				// Leave '{' for the brace-depth handler at the top of the loop.
			}
			continue;
		}

		++Pos;
	}

	return Result;
}

bool FHlslExternalPinParser::FunctionExistsInText(FStringView SourceText, FString const& FunctionName)
{
	const FString Stripped = HlslExternalPinParser::StripComments(SourceText);
	const int32 Len = Stripped.Len();
	int32 SearchFrom = 0;
	while (true)
	{
		const int32 Found = Stripped.Find(FunctionName, ESearchCase::CaseSensitive, ESearchDir::FromStart, SearchFrom);
		if (Found == INDEX_NONE)
		{
			return false;
		}

		const bool bPrecededOk = (Found == 0) || !TChar<TCHAR>::IsIdentifier(Stripped[Found - 1]);
		int32 AfterMatchPos = Found + FunctionName.Len();

		while (AfterMatchPos < Len && FChar::IsWhitespace(Stripped[AfterMatchPos]))
		{
			++AfterMatchPos;
		}
		if (bPrecededOk && AfterMatchPos < Len && Stripped[AfterMatchPos] == TEXT('('))
		{
			return true;
		}

		SearchFrom = Found + 1;
	}
}
