// Copyright Epic Games, Inc. All Rights Reserved.

#include "ComputeFramework/KernelPinCustomization.h"

#include "ComputeFramework/ComputeDataInterface.h"
#include "ComputeFramework/EditableComputeGraph.h"
#include "ComputeFramework/HlslExternalPinParser.h"
#include "ComputeFramework/ShaderParamTypeDefinition.h"
#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "IDetailChildrenBuilder.h"
#include "PropertyHandle.h"
#include "Styling/AppStyle.h"
#include "Widgets/Input/SComboBox.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "ComputeFrameworkEditor"

TSharedRef<IPropertyTypeCustomization> FKernelPinCustomization::MakeInstance()
{
	return MakeShared<FKernelPinCustomization>();
}

void FKernelPinCustomization::CustomizeHeader(TSharedRef<IPropertyHandle> StructPropertyHandle,	FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& CustomizationUtils)
{
	StructHandle = StructPropertyHandle;
	KernelFnHandle = StructPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FKernelPin, KernelFunctionName));
	InterfaceNameHandle = StructPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FKernelPin, DataInterfaceName));
	FunctionNameHandle = StructPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FKernelPin, DataInterfaceFunctionName));

	// Determine direction from the name of the containing array property.
	TSharedPtr<IPropertyHandle> ArrayHandle = StructPropertyHandle->GetParentHandle();
	bIsInput = ArrayHandle.IsValid() && ArrayHandle->GetProperty() && ArrayHandle->GetProperty()->GetFName() == GET_MEMBER_NAME_CHECKED(FComputeGraphKernelDesc, Inputs);

	// Navigate up one more level to the FComputeGraphKernelDesc element for source text access.
	TSharedPtr<IPropertyHandle> KernelDescHandle = ArrayHandle.IsValid() ? ArrayHandle->GetParentHandle() : nullptr;
	if (KernelDescHandle.IsValid())
	{
		KernelSourceHandle = KernelDescHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FComputeGraphKernelDesc, SourceText));
		KernelEntryPointHandle = KernelDescHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FComputeGraphKernelDesc, EntryPoint));
	}

	OrphanedHandle = StructPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FKernelPin, bOrphaned));

	HeaderRow
		.NameContent()
		[
			StructPropertyHandle->CreatePropertyNameWidget()
		]
		.ValueContent()
		.MinDesiredWidth(300.f)
		[
			SNew(STextBlock)
			.Text(this, &FKernelPinCustomization::GetHeaderSummaryText)
			.Font(CustomizationUtils.GetRegularFont())
			.ColorAndOpacity(this, &FKernelPinCustomization::GetHeaderSummaryColor)
			.ToolTipText(this, &FKernelPinCustomization::GetOrphanedTooltipText)
		];
}

void FKernelPinCustomization::CustomizeChildren(TSharedRef<IPropertyHandle> StructPropertyHandle, IDetailChildrenBuilder& ChildBuilder, IPropertyTypeCustomizationUtils& CustomizationUtils)
{
	// The KernelFunctionName combo box is populated from parsed HLSL pin declarations.
	ChildBuilder.AddCustomRow(LOCTEXT("KernelFunctionName", "Kernel Function"))
		.NameContent()
		[
			KernelFnHandle->CreatePropertyNameWidget()
		]
		.ValueContent()
		.MinDesiredWidth(200.f)
		[
			SNew(SComboBox<TSharedPtr<FString>>)
			.OptionsSource(&KernelFunctionNameOptions)
			.OnComboBoxOpening(this, &FKernelPinCustomization::RefreshKernelFunctionNameOptions)
			.OnSelectionChanged(this, &FKernelPinCustomization::OnKernelFunctionSelected)
			.OnGenerateWidget_Lambda([](TSharedPtr<FString> Item) -> TSharedRef<SWidget>
			{
				return SNew(STextBlock)
					.Text(FText::FromString(Item.IsValid() ? *Item : FString()))
					.Font(FAppStyle::GetFontStyle("PropertyWindow.NormalFont"));
			})
			[
				SNew(STextBlock)
				.Text(this, &FKernelPinCustomization::GetCurrentKernelFunctionNameText)
				.Font(FAppStyle::GetFontStyle("PropertyWindow.NormalFont"))
			]
		];

	// DataInterfaceName combo box.
	ChildBuilder.AddCustomRow(LOCTEXT("DataInterfaceName", "Data Interface"))
		.NameContent()
		[
			InterfaceNameHandle->CreatePropertyNameWidget()
		]
		.ValueContent()
		.MinDesiredWidth(200.f)
		[
			SNew(SComboBox<TSharedPtr<FName>>)
			.OptionsSource(&InterfaceNameOptions)
			.OnComboBoxOpening(this, &FKernelPinCustomization::RefreshInterfaceNameOptions)
			.OnSelectionChanged(this, &FKernelPinCustomization::OnInterfaceSelected)
			.OnGenerateWidget_Lambda([](TSharedPtr<FName> Item) -> TSharedRef<SWidget>
			{
				return SNew(STextBlock)
					.Text(FText::FromName(Item.IsValid() ? *Item : NAME_None))
					.Font(FAppStyle::GetFontStyle("PropertyWindow.NormalFont"));
			})
			[
				SNew(STextBlock)
				.Text(this, &FKernelPinCustomization::GetCurrentInterfaceNameText)
				.Font(FAppStyle::GetFontStyle("PropertyWindow.NormalFont"))
			]
		];

	// DataInterfaceFunctionName combo box.
	ChildBuilder.AddCustomRow(LOCTEXT("DataInterfaceFunctionName", "Function"))
		.NameContent()
		[
			FunctionNameHandle->CreatePropertyNameWidget()
		]
		.ValueContent()
		.MinDesiredWidth(200.f)
		[
			SNew(SComboBox<TSharedPtr<FString>>)
			.OptionsSource(&FunctionNameOptions)
			.OnComboBoxOpening(this, &FKernelPinCustomization::RefreshFunctionNameOptions)
			.OnSelectionChanged(this, &FKernelPinCustomization::OnFunctionSelected)
			.OnGenerateWidget_Lambda([](TSharedPtr<FString> Item) -> TSharedRef<SWidget>
			{
				return SNew(STextBlock)
					.Text(FText::FromString(Item.IsValid() ? *Item : FString()))
					.Font(FAppStyle::GetFontStyle("PropertyWindow.NormalFont"));
			})
			[
				SNew(STextBlock)
				.Text(this, &FKernelPinCustomization::GetCurrentFunctionNameText)
				.Font(FAppStyle::GetFontStyle("PropertyWindow.NormalFont"))
			]
		];
}

void FKernelPinCustomization::RefreshKernelFunctionNameOptions()
{
	KernelFunctionNameOptions.Reset();

	if (!KernelSourceHandle.IsValid() || !KernelEntryPointHandle.IsValid())
	{
		return;
	}

	FString SourceText, EntryPoint;
	if (KernelSourceHandle->GetValue(SourceText) != FPropertyAccess::Success || SourceText.IsEmpty())
	{
		return;
	}
	KernelEntryPointHandle->GetValue(EntryPoint);

	const TArray<FHlslExternalPinParser::FPinDeclaration> Decls = FHlslExternalPinParser::FindExternalPins(SourceText);
	for (FHlslExternalPinParser::FPinDeclaration const& Decl : Decls)
	{
		// Input pins read from the data interface (non-void return).
		// Output pins write (void return).
		const bool bWantOutput = !bIsInput;
		if (Decl.bIsOutput == bWantOutput)
		{
			KernelFunctionNameOptions.Add(MakeShared<FString>(Decl.FunctionName));
		}
	}
}

void FKernelPinCustomization::RefreshInterfaceNameOptions()
{
	InterfaceNameOptions.Reset();

	if (!StructHandle.IsValid())
	{
		return;
	}

	TArray<UObject*> OuterObjects;
	StructHandle->GetOuterObjects(OuterObjects);
	if (OuterObjects.Num() != 1)
	{
		return;
	}

	UEditableComputeGraph const* Graph = Cast<UEditableComputeGraph>(OuterObjects[0]);
	if (!Graph)
	{
		return;
	}

	for (FComputeGraphDataInterfaceDesc const& InterfaceDesc : Graph->GetGraphDescription().DataInterfaces)
	{
		if (!InterfaceDesc.Name.IsNone())
		{
			InterfaceNameOptions.Add(MakeShared<FName>(InterfaceDesc.Name));
		}
	}
}

void FKernelPinCustomization::RefreshFunctionNameOptions()
{
	FunctionNameOptions.Reset();

	if (!StructHandle.IsValid() || !InterfaceNameHandle.IsValid())
	{
		return;
	}

	FName CurrentInterfaceName;
	if (InterfaceNameHandle->GetValue(CurrentInterfaceName) != FPropertyAccess::Success || CurrentInterfaceName.IsNone())
	{
		return;
	}

	TArray<UObject*> OuterObjects;
	StructHandle->GetOuterObjects(OuterObjects);
	if (OuterObjects.Num() != 1)
	{
		return;
	}

	UEditableComputeGraph const* Graph = Cast<UEditableComputeGraph>(OuterObjects[0]);
	if (!Graph)
	{
		return;
	}

	for (FComputeGraphDataInterfaceDesc const& InterfaceDesc : Graph->GetGraphDescription().DataInterfaces)
	{
		if (InterfaceDesc.Name != CurrentInterfaceName)
		{
			continue;
		}

		// Prefer any configured Settings, but fall back to the CDO.
		UComputeDataInterface const* Interface = InterfaceDesc.Settings
			? InterfaceDesc.Settings.Get()
			: (InterfaceDesc.Type ? Cast<UComputeDataInterface>(InterfaceDesc.Type->GetDefaultObject()) : nullptr);

		if (!Interface)
		{
			break;
		}

		TArray<FShaderFunctionDefinition> Functions;
		if (bIsInput)
		{
			Interface->GetSupportedInputs(Functions);
		}
		else
		{
			Interface->GetSupportedOutputs(Functions);
		}

		for (FShaderFunctionDefinition const& Fn : Functions)
		{
			FunctionNameOptions.Add(MakeShared<FString>(Fn.Name));
		}
		break;
	}
}

void FKernelPinCustomization::OnKernelFunctionSelected(TSharedPtr<FString> NewValue, ESelectInfo::Type SelectInfo)
{
	if (KernelFnHandle.IsValid() && NewValue.IsValid())
	{
		KernelFnHandle->SetValue(*NewValue);
	}
}

void FKernelPinCustomization::OnInterfaceSelected(TSharedPtr<FName> NewValue, ESelectInfo::Type SelectInfo)
{
	if (InterfaceNameHandle.IsValid() && NewValue.IsValid())
	{
		InterfaceNameHandle->SetValue(*NewValue);
		
		// Clear the function name. It belonged to the old interface.
		if (FunctionNameHandle.IsValid())
		{
			FunctionNameHandle->SetValue(FString());
		}
	}
}

void FKernelPinCustomization::OnFunctionSelected(TSharedPtr<FString> NewValue, ESelectInfo::Type SelectInfo)
{
	if (FunctionNameHandle.IsValid() && NewValue.IsValid())
	{
		FunctionNameHandle->SetValue(*NewValue);
	}
}

FSlateColor FKernelPinCustomization::GetHeaderSummaryColor() const
{
	return NeedsAttention() ? FSlateColor(FLinearColor(1.f, 0.4f, 0.2f, 1.f)) : FSlateColor::UseForeground();
}

FText FKernelPinCustomization::GetOrphanedTooltipText() const
{
	if (!NeedsAttention())
	{
		return FText::GetEmpty();
	}

	bool bOrphaned = false;
	if (OrphanedHandle.IsValid())
	{
		OrphanedHandle->GetValue(bOrphaned);
	}

	return bOrphaned
		? LOCTEXT("OrphanedPinTooltip", "This pin's kernel function no longer appears in the HLSL. Update or remove the binding.")
		: LOCTEXT("IncompletePinTooltip", "This pin is not fully configured. Set the kernel function, data interface, and data interface function.");
}

bool FKernelPinCustomization::NeedsAttention() const
{
	FString KernelFn;
	if (KernelFnHandle.IsValid() && KernelFnHandle->GetValue(KernelFn) == FPropertyAccess::Success && KernelFn.IsEmpty())
	{
		return true;
	}

	FName InterfaceName;
	if (InterfaceNameHandle.IsValid() && InterfaceNameHandle->GetValue(InterfaceName) == FPropertyAccess::Success && InterfaceName.IsNone())
	{
		return true;
	}

	FString FunctionName;
	if (FunctionNameHandle.IsValid() && FunctionNameHandle->GetValue(FunctionName) == FPropertyAccess::Success && FunctionName.IsEmpty())
	{
		return true;
	}

	bool bOrphaned = false;
	if (OrphanedHandle.IsValid()) 
	{
		OrphanedHandle->GetValue(bOrphaned);
	}
	return bOrphaned;
}

FText FKernelPinCustomization::GetHeaderSummaryText() const
{
	if (!KernelFnHandle.IsValid())
	{
		return FText::GetEmpty();
	}

	FString KernelFn;
	KernelFnHandle->GetValue(KernelFn);
	
	FName InterfaceName;
	InterfaceNameHandle->GetValue(InterfaceName);
	
	FString InterfaceFn;
	FunctionNameHandle->GetValue(InterfaceFn);

	if (KernelFn.IsEmpty() && InterfaceName.IsNone() && InterfaceFn.IsEmpty())
	{
		return LOCTEXT("EmptyPin", "(empty)");
	}

	const FString Arrow = bIsInput ? TEXT(" \u2190 ") : TEXT(" \u2192 ");
	FString Summary = KernelFn + Arrow + InterfaceName.ToString();
	if (!InterfaceFn.IsEmpty())
	{
		Summary += TEXT(".") + InterfaceFn;
	}
	return FText::FromString(Summary);
}

FText FKernelPinCustomization::GetCurrentKernelFunctionNameText() const
{
	FString Value;
	if (KernelFnHandle.IsValid() && KernelFnHandle->GetValue(Value) == FPropertyAccess::Success)
	{
		if (!Value.IsEmpty())
		{
			return FText::FromString(Value);
		}
	}
	return LOCTEXT("SelectKernelFn", "(select)");
}

FText FKernelPinCustomization::GetCurrentInterfaceNameText() const
{
	FName Value;
	if (InterfaceNameHandle.IsValid() && InterfaceNameHandle->GetValue(Value) == FPropertyAccess::Success)
	{
		if (!Value.IsNone())
		{
			return FText::FromName(Value);
		}
	}
	return LOCTEXT("SelectInterface", "(select)");
}

FText FKernelPinCustomization::GetCurrentFunctionNameText() const
{
	FString Value;
	if (FunctionNameHandle.IsValid() && FunctionNameHandle->GetValue(Value) == FPropertyAccess::Success)
	{
		if (!Value.IsEmpty())
		{
			return FText::FromString(Value);
		}
	}
	return LOCTEXT("SelectFunction", "(select)");
}

#undef LOCTEXT_NAMESPACE
